// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
#if !__MonoCS__
using System.Deployment.Application;
#endif
using System.Diagnostics;
using System.IO;
using System.Net;
using System.Net.Sockets;
using System.Reflection;
using System.Runtime.Remoting;
using System.Runtime.Remoting.Channels;
using System.Runtime.Remoting.Channels.Ipc;
using System.Runtime.Remoting.Channels.Tcp;
using System.Runtime.Remoting.Lifetime;
using System.Runtime.Serialization.Formatters.Binary;
using System.Security.Cryptography.X509Certificates;
using System.Threading;
using System.Text;

using AgentInterface;
using SwarmCoordinatorInterface;

namespace Agent
{
	public interface IAgentInternalInterface
	{
		/**
		 * Init the local agent
		 */
		bool Init( Int32 NewAgentProcessID );

		/**
		 * Cleanup the agent on shutdown
		 */
		void Destroy();

		/**
		 * Returns true if the Agent has recieved a signal to shutdown, typically either
		 * from the coordinator or from the local user via the GUI
		 */
		bool ShuttingDown();

		/**
		 * Returns true if the agent is restarting due to a command from the coordinator
		 */
		bool Restarting();

		/**
		 * Used to signal the Agent that a shutdown has been requested
		 */
		void RequestShutdown();

		/**
		 * Part of the main loop
		 */
		void MaintainAgent();

		/**
		 * Part of the main loop
		 */
		void MaintainConnections();

		/**
		 * Part of the main loop
		 */
		void MaintainCache();

		/**
		 * Part of the main loop
		 */
		void MaintainJobs();

		/**
		 * Called by a remote Agent, this provides the data for a new channel being pushed
		 */
		bool SendChannel( Int32 ConnectionHandle, string ChannelName, byte[] ChannelData, AgentGuid JobGuid );

		/**
		 * Called by a remote Agent, this requests that a specified channel be pushed back
		 */
		bool RequestChannel( Int32 ConnectionHandle, string ChannelName, AgentGuid JobGuid );

		/**
		 * Used to validate a remote channel by providing a remotely generated hash that
		 * we'll use to compare against a locally generated hash - return value is
		 * whether they match
		 */
		bool ValidateChannel( Int32 ConnectionHandle, string ChannelName, byte[] RemoteChannelHashValue );

		/**
		 * A simple method used to test the validity of the remoting connection
		 */
		bool TestConnection( Int32 ConnectionHandle );
	}

	/**
	 * Header that comes at the beginning of a file transfer
	 */
	[Serializable]
	public struct ChannelTransferData
	{
		public string	ChannelName;
		public byte[]	ChannelData;
	}

	/**
	 * Implementation of the AgentInterface, primary service class for the Swarm system.
	 * Note that much of the implementation is spread across multiple files, by general
	 * subsystem (channels, messaging, connections, jobs, etc.).
	 */
	public partial class Agent : MarshalByRefObject, IAgentInternalInterface, IAgentInterface
	{
		///////////////////////////////////////////////////////////////////////

		/**
		 * The process ID of the last process to call Init
		 */
		public Int32 AgentProcessID = -1;
		public Int32 OwnerProcessID = -1;

		/**
		 * The X.509 Certificate information for the executing application
		 */
		public X509Certificate Certificate = null;

		/**
		 * A very simple "has shutdown been requested" variable used to communicate
		 * with worker threads about the current state of the Agent object
		 */
		public bool AgentIsShuttingDown = false;
		public bool AgentHasShutDown = false;

		/**
		 * Thread used to process messages in the Agent's message queue
		 */
		private Thread ProcessMessageThread;

		/**
		 * Signal that the Agent has been fully initialized and when
		 */
		private ManualResetEvent Initialized = new ManualResetEvent( false );
		private DateTime InitializedTime;

		/**
		 * The next times we should run idle processing loop tasks
		 */
		private DateTime NextCleanUpCacheTime;

        /**
		 * Info about the local agent used for matching up jobs
         */
        private Version CurrentVersion = Assembly.GetExecutingAssembly().GetName().Version;

		/**
		 * A collection of simple performance counters used to get basic monitoring stats
		 */
		private class PerformanceStats
		{
			public PerformanceStats()
			{
                // Allocate the counters
                if( AgentApplication.DeveloperOptions.LocalEnableLocalPerformanceMonitoring )
                {
                    CPUBusy = new PerformanceCounter( "Processor", "% Processor Time", "_Total" );
                }

				// Start out pessimistic
				for( int i = 0; i < AverageCPUBusyValuesCount; i++ )
				{
					AverageCPUBusyValues[i] = 100.0f;
					AverageCPUBusyRaw += 100.0f;
				}
				AverageCPUBusyIndex = 0;
				AverageCPUBusy = AverageCPUBusyRaw / AverageCPUBusyValuesCount;
	
				// Initialize the counters
				Update();
			}

			public bool Update()
			{
				// If it's time for an update, do one
				bool ValuesUpdated = false;
				if( DateTime.UtcNow > NextUpdateTime )
				{
					// Get the latest CPU values
                    LastCPUBusy = CPUBusy != null ? CPUBusy.NextValue() : 100.0f;
		
					// Update the raw running average
					AverageCPUBusyRaw -= AverageCPUBusyValues[AverageCPUBusyIndex];
					AverageCPUBusyRaw += LastCPUBusy;

					// Update the running average array
					AverageCPUBusyValues[AverageCPUBusyIndex] = LastCPUBusy;
					AverageCPUBusyIndex = ( AverageCPUBusyIndex + 1 ) % AverageCPUBusyValuesCount;

					// Update the current average value
					AverageCPUBusy = AverageCPUBusyRaw / AverageCPUBusyValuesCount;

					// Update the next update time
                    NextUpdateTime = DateTime.UtcNow + TimeSpan.FromSeconds(1);
					ValuesUpdated = true;
				}
				return ValuesUpdated;
			}

			private DateTime NextUpdateTime = DateTime.UtcNow + TimeSpan.FromSeconds(1);

			// CPU stats (default to maximum busy)
            private PerformanceCounter CPUBusy = null;
			public float LastCPUBusy = 100.0f;

			private const int AverageCPUBusyValuesCount = 10;
			private float[] AverageCPUBusyValues = new float[AverageCPUBusyValuesCount];
			private int AverageCPUBusyIndex;
			public float AverageCPUBusyRaw;
			public float AverageCPUBusy;

			// TODO: More stats (see the Builder Controller for examples)
		};
		PerformanceStats LocalPerformanceStats = null;
		float CPUBusyThreshold = 20.0f;

		///////////////////////////////////////////////////////////////////////

		public override Object InitializeLifetimeService()
		{
			// We don't ever want this to expire...
			ILease NewLease = ( ILease )base.InitializeLifetimeService();
			if( NewLease.CurrentState == LeaseState.Initial )
			{
				NewLease.InitialLeaseTime = TimeSpan.Zero;
			}
			return NewLease;
		}

		public bool Init( Int32 NewAgentProcessID )
		{
			Log( EVerbosityLevel.Verbose, ELogColour.Green, " ...... checking certificate" );
			string AgentFileName = Assembly.GetExecutingAssembly().Location;
			try
			{
				Certificate = X509Certificate.CreateFromSignedFile( AgentFileName );
			}
			catch( Exception )
			{
				// Any exception means that either the file isn't signed or has an invalid certificate
			}
			if( Certificate != null )
			{
				// If we have a valid certificate, do the rest of the security checks
				bool bSecurityCheckPassed = false;

				// Check the certificate for some basic information to make sure it's ours
				byte[] EpicCertificateSerialNumber = { 0x42, 0x58, 0xa1, 0xd9, 0x82, 0x4b, 0x70, 0xe5, 0x07, 0x19, 0x96, 0xd8, 0xda, 0xcd, 0x16, 0x1c };
				byte[] OtherCertificateSerialNumber = Certificate.GetSerialNumber();
				if( EpicCertificateSerialNumber.Length == OtherCertificateSerialNumber.Length )
				{
					bool bSerialNumbersAreEqual = true;
					for( Int32 Index = 0; Index < EpicCertificateSerialNumber.Length; Index++ )
					{
						bSerialNumbersAreEqual &= ( EpicCertificateSerialNumber[Index] == OtherCertificateSerialNumber[Index] );
					}

					// If the certificate checks out, move onto validating our known libraries
					if( bSerialNumbersAreEqual )
					{
						bool bLibrariesAreOkay = true;
						try
						{
							X509Certificate NextCertificate;
							string BasePath = Path.GetDirectoryName( AgentFileName );
							string[] LibrariesToCheck = {
								Path.Combine( BasePath, "AgentInterface.dll" ),
								Path.Combine( BasePath, "UnrealControls.dll" ),
								Path.Combine( BasePath, "SwarmCoordinatorInterface.dll" ),
							};

							foreach( string NextLibrary in LibrariesToCheck )
							{
								NextCertificate = X509Certificate.CreateFromSignedFile( NextLibrary );
								bLibrariesAreOkay &= NextCertificate.Equals( Certificate );
							}
						}
						catch( Exception )
						{
							// If any of them fail, they all fail
							bLibrariesAreOkay = false;
						}

						// If we get here and the libraries are okay, we're done validating
						if( bLibrariesAreOkay )
						{
							bSecurityCheckPassed = true;
						}
					}
				}

				if( bSecurityCheckPassed == false )
				{
					// Not what we expect, ditch it
					Certificate = null;
				}
			}
			if( Certificate == null )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Orange, " ......... certificate check has failed" );
			}

			Log( EVerbosityLevel.Informative, ELogColour.Green, " ...... initializing cache" );
			if( InitCache() == false )
			{
				// Failed to initialize the cache properly, fail
				return false;
			}

			// Initialize the coordinator connection
			Log( EVerbosityLevel.Informative, ELogColour.Green, " ...... initializing connection to SwarmCoordinator" );
            InitCoordinator();

			// Initialize the local performance monitoring object
			Log( EVerbosityLevel.Informative, ELogColour.Green, " ...... initializing local performance monitoring subsystem" );
			try
			{
				LocalPerformanceStats = new PerformanceStats();
			}
			catch ( Exception Ex )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Orange, " ...... local performance monitoring subsystem initialization failed" );
				Log( EVerbosityLevel.Verbose, ELogColour.Orange, Ex.ToString() );
			}

			// Startup the message processing thread
			ThreadStart ThreadStartDelegateMessages = new ThreadStart( ProcessMessagesThreadProc );
			ProcessMessageThread = new Thread( ThreadStartDelegateMessages );
			ProcessMessageThread.Name = "ProcessMessageThread";
			ProcessMessageThread.Start();

			// Set the next times we should run some of the idle task processors
			NextCleanUpCacheTime = DateTime.UtcNow + TimeSpan.FromSeconds(30);

			// Set the owning process ID
			AgentProcessID = NewAgentProcessID;

			// Finally, signal that we're fully initialized
			InitializedTime = DateTime.UtcNow;
			Initialized.Set();

			return true;
		}

		public void Destroy()
		{
			// We should already have received this signal
			Debug.Assert( AgentIsShuttingDown );

			// Close any open connections
			Hashtable InParameters = null;
			Hashtable OutParameters = null;
			foreach( Connection NextConnection in Connections.Values )
			{
				CloseConnection( NextConnection.Handle, InParameters, ref OutParameters );
			}

			AgentHasShutDown = true;

			// Update our state and notify the coordinator
			CurrentState = AgentState.Closed;
			WorkingFor = "";
			PingCoordinator( true );
		}

		public void MaintainAgent()
		{
#if !__MonoCS__
			// We'll only care about this if we're running on a user's machine and assume
			// that on the swarm farm machines, it's the only thing running
			if( ApplicationDeployment.IsNetworkDeployed == false )
#endif
			{
				// Update the local performance stats
				if( ( LocalPerformanceStats != null ) &&
					( LocalPerformanceStats.Update() ) )
				{
					// Note that we're quick to become Busy, slow to return to Available
					if( LocalPerformanceStats.LastCPUBusy > CPUBusyThreshold )
					{
						if( CurrentState == AgentState.Available )
						{
							CurrentState = AgentState.Busy;
							PingCoordinator( true );
						}
					}
					else if( LocalPerformanceStats.AverageCPUBusy <= ( CPUBusyThreshold * 0.5f ) )
					{
						if( CurrentState == AgentState.Busy )
						{
							CurrentState = AgentState.Available;
							PingCoordinator( true );
						}
					}
				}
			}

			// Determine if our owning process is still alive, if we have one
			if( OwnerProcessID > 0 )
			{
				bool bProcessIsStillRunning = false;
				Process ProcessObject = null;
				try
				{
					ProcessObject = Process.GetProcessById( OwnerProcessID );
					if( ProcessObject.HasExited == false )
					{
						// Connection is still alive
						bProcessIsStillRunning = true;
					}
				}
				catch( Exception )
				{
					// Process no longer exists, fall through
				}

				// Owner process is gone, time to shut down
				if( bProcessIsStillRunning == false )
				{
					RequestShutdown();
				}
			}
		}

		///////////////////////////////////////////////////////////////////////

		public bool TestConnection( Int32 ConnectionHandle )
		{
			// Return whether we think this is a valid connection or not
			return Connections.ContainsKey( ConnectionHandle );
		}

		///////////////////////////////////////////////////////////////////////

		// Standard constructor
		public Agent()
		{
		}
	}
}
