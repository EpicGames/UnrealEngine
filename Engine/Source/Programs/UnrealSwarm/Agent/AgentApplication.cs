// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
#if !__MonoCS__
using System.Deployment.Application;
#endif
using System.Diagnostics;
using System.Runtime.Remoting;
using System.Runtime.Remoting.Channels;
using System.Runtime.Remoting.Channels.Ipc;
using System.Runtime.Remoting.Channels.Tcp;
using System.Runtime.Remoting.Lifetime;
using System.Threading;
using System.Text;
using System.Windows.Forms;

using AgentInterface;

namespace Agent
{
	static partial class AgentApplication
	{
		// Interface to our local agent
		private static Agent LocalAgent = null;

		// For deployed applications, we'll ocassionally check for updates
        private static DateTime NextUpdateCheckTime = DateTime.UtcNow;

		/**
		 * Checks for any pending updates for this application, and if it finds
		 * any, updates the application as part of a restart
		 */
        static bool CheckForUpdates()
		{
#if !__MonoCS__
			try
			{
				if( ApplicationDeployment.IsNetworkDeployed )
				{
					ApplicationDeployment Current = ApplicationDeployment.CurrentDeployment;
					return Current.CheckForUpdate();
				}
			}
			catch( Exception )
			{
			}
#endif
			return false;
		}
		static bool InstallAllUpdates()
		{
#if !__MonoCS__
			try
			{
				if( ApplicationDeployment.IsNetworkDeployed )
				{
					ApplicationDeployment Current = ApplicationDeployment.CurrentDeployment;

					// If there are any updates available, install them now
					if( Current.CheckForUpdate() )
					{
						return Current.Update();
					}
				}
			}
			catch( Exception )
			{
			}
#endif
			return false;
		}

		/*
		 * Delete the swarm cache whenever the agents are restarted (to avoid version problems)
		 */
		static void ClearCache()
		{
			string CacheFolder = Options.CacheFolder;

			try
			{
				if( Directory.Exists( CacheFolder ) )
				{
					Directory.Delete( CacheFolder, true );
				}
			}
			catch
			{
			}
		}

		/**
		 * The main entry point for the application
		 */
        [STAThread]
		static void Main( string[] args )
		{
            // Read args
            ParseArgs(args);

            // Start up the GUI thread
            InitGUIThread();

			AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "Starting up SwarmAgent ..." );
			AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, " ... registering SwarmAgent with remoting service" );

			// Register the local agent singleton
			RemotingConfiguration.RegisterWellKnownServiceType( typeof( Agent ), "SwarmAgent", WellKnownObjectMode.Singleton );

            AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, " ... registering SwarmAgent network channels" );

			// We're going to have two channels for the Agent: one for the actual Agent
			// application (IPC with infinite timeout) and one for all other remoting
			// traffic (TCP with infinite timeout that we monitor for drops)
			IpcChannel AgentIPCChannel = null;
            TcpChannel AgentTCPChannel = null;
            while( ( Ticking == true ) &&
				   ( ( AgentIPCChannel == null ) ||
				     ( AgentTCPChannel == null ) ) )
			{
				try
				{
					if( AgentIPCChannel == null )
					{
						// Register the IPC connection to the local agent
						string IPCChannelPortName = String.Format( "127.0.0.1:{0}", Properties.Settings.Default.AgentRemotingPort );
						AgentIPCChannel = new IpcChannel( IPCChannelPortName );
						ChannelServices.RegisterChannel( AgentIPCChannel, false );
					}

					if( AgentTCPChannel == null )
					{
						// Register the TCP connection to the local agent
						AgentTCPChannel = new TcpChannel( Properties.Settings.Default.AgentRemotingPort );
						ChannelServices.RegisterChannel( AgentTCPChannel, false );
					}
				}
				catch (RemotingException Ex)
				{
					AgentApplication.Log(EVerbosityLevel.Informative, ELogColour.Orange, "[ERROR] Channel already registered, suggesting another SwarmAgent or client is running.");
					AgentApplication.Log(EVerbosityLevel.Informative, ELogColour.Orange, "[ERROR] If you feel this is in error, check your running process list for additional copies of");
					AgentApplication.Log(EVerbosityLevel.Informative, ELogColour.Orange, "[ERROR] SwarmAgent or UnrealLightmass (or other client) and consider killing them.");
					AgentApplication.Log(EVerbosityLevel.Informative, ELogColour.Orange, "[ERROR] Sleeping for a few seconds and trying again...");
					AgentApplication.Log(EVerbosityLevel.Informative, ELogColour.Orange, string.Format("[ERROR] Channel registration failed. Reason: {0}\n, Callstack: {1}.", Ex.Message, Ex.StackTrace));
					Thread.Sleep(3000);
				}
				catch (Exception Ex)
				{
					AgentApplication.Log(EVerbosityLevel.Informative, ELogColour.Orange, string.Format("[ERROR] Channel registration failed. Reason: {0}\n, Callstack: {1}.", Ex.Message, Ex.StackTrace));
					Thread.Sleep(3000);
				}
			}

			// if we're still ticking, we should have both of our channels initialized
			if( Ticking )
			{
				Debug.Assert( AgentIPCChannel != null );
				Debug.Assert( AgentTCPChannel != null );
			}
			else
			{
				// Otherwise, we can simply return to exit
				return;
			}

			// Get the agent interface object using the IPC channel
			string LocalAgentURL = String.Format( "ipc://127.0.0.1:{0}/SwarmAgent", Properties.Settings.Default.AgentRemotingPort.ToString() );
			LocalAgent = ( Agent )Activator.GetObject( typeof( Agent ), LocalAgentURL );

			AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, " ... initializing SwarmAgent" );
            
            // Init the local agent object (if this is the first call to it, it will be created now)
			bool AgentInitializedSuccessfully = false;
			try
			{
				AgentInitializedSuccessfully = LocalAgent.Init( Process.GetCurrentProcess().Id );
			}
			catch( Exception Ex )
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Red, "[ERROR] Local agent failed to initialize with an IPC channel! Falling back to TCP..." );
				AgentApplication.Log( EVerbosityLevel.Verbose, ELogColour.Red, "[ERROR] Exception details: " + Ex.ToString() );

				// Try again with the TCP channel, which is slower but should work
				LocalAgentURL = String.Format( "tcp://127.0.0.1:{0}/SwarmAgent", Properties.Settings.Default.AgentRemotingPort.ToString() );
				LocalAgent = ( Agent )Activator.GetObject( typeof( Agent ), LocalAgentURL );

				try
				{
					AgentInitializedSuccessfully = LocalAgent.Init( Process.GetCurrentProcess().Id );
					if( AgentInitializedSuccessfully )
					{
						AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Red, "[ERROR] RECOVERED by using TCP channel!" );
					}
				}
				catch( Exception Ex2 )
				{
					AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Red, "[ERROR] Local agent failed to initialize with a TCP channel! Fatal error." );
					AgentApplication.Log( EVerbosityLevel.Verbose, ELogColour.Red, "[ERROR] Exception details: " + Ex2.ToString() );

					ChannelServices.UnregisterChannel( AgentTCPChannel );
					ChannelServices.UnregisterChannel( AgentIPCChannel );
					return;
				}
			}

			// Only continue if we have a fully initialized agent
			if( ( LocalAgent != null ) && AgentInitializedSuccessfully )
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, " ... initialization successful, SwarmAgent now running" );
                
                // Loop until a quit/restart event has been requested
				while( !LocalAgent.ShuttingDown() )
				{
					try
					{
						// If we've stopped ticking, notify the agent that we want to shutdown
						if( !Ticking )
						{
							LocalAgent.RequestShutdown();
						}

						// Maintain the agent itself
						LocalAgent.MaintainAgent();

						// Maintain any running active connections
						LocalAgent.MaintainConnections();

						// Maintain the Agent's cache
						if( CacheRelocationRequested )
						{
							LocalAgent.RequestCacheRelocation();
							CacheRelocationRequested = false;
						}
						if( CacheClearRequested )
						{
							LocalAgent.RequestCacheClear();
							CacheClearRequested = false;
						}
						if( CacheValidateRequested )
						{
							LocalAgent.RequestCacheValidate();
							CacheValidateRequested = false;
						}
						LocalAgent.MaintainCache();

						// Maintain any running jobs
						LocalAgent.MaintainJobs();

						// If this is a deployed application which is configured to auto-update,
						// we'll check for any updates and, if there are any, request a restart
						// which will install them prior to restarting
#if !__MonoCS__
						if( ( AgentApplication.DeveloperOptions.UpdateAutomatically ) &&
							( ApplicationDeployment.IsNetworkDeployed ) &&
                            (DateTime.UtcNow > NextUpdateCheckTime))
						{
							if( CheckForUpdates() )
							{
								LocalAgent.RequestRestart();
							}
                            NextUpdateCheckTime = DateTime.UtcNow + TimeSpan.FromMinutes(1);
						}
#endif
					}
					catch( Exception Ex )
					{
                        AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Red, "[ERROR] UNHANDLED EXCEPTION: " + Ex.Message );
						AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Red, "[ERROR] UNHANDLED EXCEPTION: " + Ex.ToString() );
					}

					// Sleep for a little bit
					Thread.Sleep( 500 );
				}

				// Let the GUI destroy itself
				RequestQuit();
				
				bool AgentIsRestarting = LocalAgent.Restarting();

				// Do any required cleanup
				LocalAgent.Destroy();

				ChannelServices.UnregisterChannel( AgentTCPChannel );
				ChannelServices.UnregisterChannel( AgentIPCChannel );

				// Now that everything is shut down, restart if requested
				if( AgentIsRestarting )
				{
					ClearCache();
					InstallAllUpdates();
					Application.Restart();
				}
			}
		}
	}
}
