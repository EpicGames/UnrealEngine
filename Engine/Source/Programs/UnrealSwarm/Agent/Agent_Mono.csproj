<?xml version="1.0" encoding="utf-8"?>
<Project DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003" ToolsVersion="4.0">
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">AnyCPU</Platform>
    <ProductVersion>9.0.30729</ProductVersion>
    <SchemaVersion>2.0</SchemaVersion>
    <ProjectGuid>{F5352283-246B-42B4-9A35-92121BDE8652}</ProjectGuid>
    <OutputType>WinExe</OutputType>
    <AppDesignerFolder>Properties</AppDesignerFolder>
    <RootNamespace>Agent</RootNamespace>
    <AssemblyName>SwarmAgent</AssemblyName>
    <SccProjectName>SAK</SccProjectName>
    <SccLocalPath>SAK</SccLocalPath>
    <SccAuxPath>SAK</SccAuxPath>
    <SccProvider>SAK</SccProvider>
    <IsWebBootstrapper>false</IsWebBootstrapper>
    <ManifestCertificateThumbprint>E10B5956C0FAFF8479C61A48C48EFDE555A18633</ManifestCertificateThumbprint>
    <ManifestKeyFile>
    </ManifestKeyFile>
    <GenerateManifests>true</GenerateManifests>
    <SignManifests>true</SignManifests>
    <FileUpgradeFlags>
    </FileUpgradeFlags>
    <OldToolsVersion>3.5</OldToolsVersion>
    <UpgradeBackupLocation>
    </UpgradeBackupLocation>
    <PublishUrl>\\devweb-02\Deploy\Swarm\SwarmAgent\</PublishUrl>
    <Install>true</Install>
    <InstallFrom>Unc</InstallFrom>
    <UpdateEnabled>false</UpdateEnabled>
    <UpdateMode>Foreground</UpdateMode>
    <UpdateInterval>7</UpdateInterval>
    <UpdateIntervalUnits>Days</UpdateIntervalUnits>
    <UpdatePeriodically>false</UpdatePeriodically>
    <UpdateRequired>false</UpdateRequired>
    <MapFileExtensions>true</MapFileExtensions>
    <ProductName>Swarm Agent</ProductName>
    <PublisherName>Startup</PublisherName>
    <CreateWebPageOnPublish>true</CreateWebPageOnPublish>
    <WebPage>publish.htm</WebPage>
    <OpenBrowserOnPublish>false</OpenBrowserOnPublish>
    <ApplicationRevision>3</ApplicationRevision>
    <ApplicationVersion>1.7.2891.0</ApplicationVersion>
    <UseApplicationTrust>false</UseApplicationTrust>
    <PublishWizardCompleted>true</PublishWizardCompleted>
    <BootstrapperEnabled>true</BootstrapperEnabled>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <DebugSymbols>True</DebugSymbols>
    <DebugType>pdbonly</DebugType>
    <Optimize>False</Optimize>
    <OutputPath>..\..\..\..\Binaries\DotNET\Mac</OutputPath>
    <DefineConstants>TRACE;DEBUG;SPECIFIC_VERSION</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
    <UseVSHostingProcess>false</UseVSHostingProcess>
    <EnvironmentVariables>
      <EnvironmentVariables>
        <Variable name="MONO_LOG_LEVEL" value="debug" />
      </EnvironmentVariables>
    </EnvironmentVariables>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Development|AnyCPU' ">
    <DebugType>pdbonly</DebugType>
    <Optimize>True</Optimize>
    <OutputPath>..\..\..\..\Binaries\DotNET\Mac</OutputPath>
    <DefineConstants>TRACE;SPECIFIC_VERSION</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
    <PlatformTarget>AnyCPU</PlatformTarget>
    <UseVSHostingProcess>false</UseVSHostingProcess>
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="System" />
    <Reference Include="System.Core">
      <RequiredTargetFramework>3.5</RequiredTargetFramework>
    </Reference>
    <Reference Include="System.Data" />
    <Reference Include="System.Drawing" />
    <Reference Include="System.Management" />
    <Reference Include="System.Runtime.Remoting" />
    <Reference Include="System.Runtime.Serialization" />
    <Reference Include="System.ServiceModel" />
    <Reference Include="System.ServiceProcess" />
    <Reference Include="System.Windows.Forms" />
    <Reference Include="System.Xml" />
  </ItemGroup>
  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
    <Compile Include="Channels.cs" />
    <Compile Include="AgentApplication.cs" />
    <Compile Include="Connections.cs" />
    <Compile Include="Interface.cs" />
    <Compile Include="Jobs.cs" />
    <Compile Include="Messaging.cs" />
    <Compile Include="Progression.cs">
      <SubType>Form</SubType>
    </Compile>
    <Compile Include="SettableOptions.cs" />
    <Compile Include="Utils.cs" />
    <Compile Include="Display.cs">
      <SubType>Form</SubType>
    </Compile>
    <Compile Include="Display.Designer.cs">
      <DependentUpon>Display.cs</DependentUpon>
    </Compile>
    <Compile Include="Agent.cs">
    </Compile>
    <Compile Include="Properties\AssemblyInfo.cs" />
    <EmbeddedResource Include="Display.resx">
      <SubType>Designer</SubType>
      <DependentUpon>Display.cs</DependentUpon>
    </EmbeddedResource>
    <EmbeddedResource Include="Properties\Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
      <SubType>Designer</SubType>
    </EmbeddedResource>
    <Compile Include="Properties\Resources.Designer.cs">
      <AutoGen>True</AutoGen>
      <DependentUpon>Resources.resx</DependentUpon>
      <DesignTime>True</DesignTime>
    </Compile>
    <None Include="app.config" />
    <None Include="Properties\Settings.settings">
      <Generator>SettingsSingleFileGenerator</Generator>
      <LastGenOutput>Settings.Designer.cs</LastGenOutput>
    </None>
    <Compile Include="Properties\Settings.Designer.cs">
      <AutoGen>True</AutoGen>
      <DependentUpon>Settings.settings</DependentUpon>
      <DesignTimeSharedInput>True</DesignTimeSharedInput>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\UnrealControls\UnrealControls.csproj">
      <Project>{8F079097-3724-4B31-AD3B-0A5B76C26376}</Project>
      <Name>UnrealControls</Name>
    </ProjectReference>
    <ProjectReference Include="..\AgentInterface\AgentInterface_Mono.csproj">
      <Project>{6F18ADAC-2619-4C81-B80F-22927E56CA47}</Project>
      <Name>AgentInterface_Mono</Name>
    </ProjectReference>
    <ProjectReference Include="..\SwarmCommonUtils\SwarmCommonUtils.csproj">
      <Project>{65521250-b022-442f-b961-3af5ada02083}</Project>
      <Name>SwarmCommonUtils</Name>
    </ProjectReference>
    <ProjectReference Include="..\SwarmCoordinatorInterface\SwarmCoordinatorInterface_Mono.csproj">
      <Project>{4C299DD0-B968-49F2-894A-9F115C369E72}</Project>
      <Name>SwarmCoordinatorInterface_Mono</Name>
    </ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <BootstrapperPackage Include=".NETFramework,Version=v4.0,Profile=Client">
      <Visible>False</Visible>
      <ProductName>Microsoft .NET Framework 4 Client Profile %28x86 and x64%29</ProductName>
      <Install>true</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Client.3.5">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1 Client Profile</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.2.0">
      <Visible>False</Visible>
      <ProductName>.NET Framework 2.0 %28x86%29</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.0">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.0 %28x86%29</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.5">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.5.SP1">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Windows.Installer.3.1">
      <Visible>False</Visible>
      <ProductName>Windows Installer 3.1</ProductName>
      <Install>true</Install>
    </BootstrapperPackage>
  </ItemGroup>
  <ItemGroup>
    <WCFMetadata Include="Service References\" />
  </ItemGroup>
  <Import Project="$(MSBuildBinPath)\Microsoft.CSharp.targets" />
  <!-- To modify your build process, add your task inside one of the targets below and uncomment it. 
       Other similar extension points exist, see Microsoft.Common.targets.
  <Target Name="BeforeBuild">
  </Target>
  <Target Name="AfterBuild">
  </Target>
  -->
  <PropertyGroup>
    <PostBuildEvent>
    </PostBuildEvent>
  </PropertyGroup>
</Project>