// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
#if !__MonoCS__
using System.Deployment.Application;
#endif
using System.Diagnostics;
using System.IO;
using System.Net.Sockets;
using System.Runtime.Serialization.Formatters.Binary;
using System.Security.Cryptography;
using System.Threading;

using AgentInterface;

namespace Agent
{
	///////////////////////////////////////////////////////////////////////////

	public class Channel
	{
		/**
		 * Name of the channel in the cache
		 */
		private string Private_Name = "";
		public string Name
		{
			get { return Private_Name; }
			set { Private_Name = value; }
		}

		/**
		 * Any flags the channel was created with (from EChannelFlags)
		 */
		private EChannelFlags Private_Flags;
		public EChannelFlags Flags
		{
			get { return Private_Flags; }
			set { Private_Flags = value; }
		}

		/**
		 * Externally visible handle used for managing the channel
		 */
		private Int32 Private_Handle = Constants.INVALID;
		public Int32 Handle
		{
			get { return Private_Handle; }
			set { Private_Handle = value; }
		}

		/**
		 * Full path name of the channel in the cache
		 */
		private string Private_FullName = "";
		public string FullName
		{
			get { return Private_FullName; }
			set { Private_FullName = value; }
		}

		///////////////////////////////////////////////////////////////////////////

		/**
		 * Standard constructor
		 */
		public Channel( string ChannelName, string FullChannelName, EChannelFlags ChannelFlags )
		{
			Name = ChannelName;
			Flags = ChannelFlags;
			FullName = FullChannelName;
		}

		public void SetHandle( Int32 NewHandle )
		{
			Handle = NewHandle;
		}
	}

	///////////////////////////////////////////////////////////////////////////

	public class JobChannel : Channel
	{
		/**
		 * The GUID of the Job this channel belongs to
		 */
		private AgentGuid Private_JobGuid;
		public AgentGuid JobGuid
		{
			get { return Private_JobGuid; }
			set { Private_JobGuid = value; }
		}

		///////////////////////////////////////////////////////////////////////////

		/**
		 * Standard constructor
		 */
		public JobChannel( AgentGuid NewJobGuid, string ChannelName, string FullChannelName, EChannelFlags ChannelFlags )
			: base( ChannelName, FullChannelName, ChannelFlags )
		{
			JobGuid = NewJobGuid;
		}
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of channel management behavior in the Agent
	 */
	public partial class Agent : MarshalByRefObject, IAgentInternalInterface, IAgentInterface
	{
		///////////////////////////////////////////////////////////////////////////

		// Mapping from persistent cache channel names to a hash of its contents
		private ReaderWriterDictionary<string, byte[]> ChannelHashValues = new ReaderWriterDictionary<string, byte[]>();

		// Use the SHA1Managed class to create the cache channel hash values
		private SHA1Managed SHhash = new SHA1Managed();

		// Variables used to request cache cleanup actions
		bool CacheRelocationRequested = false;
		bool CacheClearRequested = false;
		bool CacheValidateRequested = false;

		///////////////////////////////////////////////////////////////////////////

		private byte[] ComputeHash( byte[] HashInput )
		{
			lock( SHhash )
			{
				return SHhash.ComputeHash( HashInput );
			}
		}

		public bool InitCache()
		{
			try
			{
				// Ensure that the folders we need are there
				string CacheFolder = GetCacheLocation();
				if( CacheFolder.Length == 0 )
				{
					Log( EVerbosityLevel.Informative, ELogColour.Red, " ......... Failed to get the cache location!" );
					return false;
				}

				Log( EVerbosityLevel.Informative, ELogColour.Green, " ......... using cache folder '" + CacheFolder + "'" );

				// The agent staging area needs to be wiped clean on start up
				Log( EVerbosityLevel.Informative, ELogColour.Green, " ......... recreating SwarmAgent cache staging area" );
				string AgentStagingArea = Path.Combine( AgentApplication.Options.CacheFolder, "AgentStagingArea" );
				DirectoryRecreate( AgentStagingArea );

#if STARTUP_GENERATE_HASHES
				// Populate the hash values table (only need to do persistent cache entries)
				Log( EVerbosityLevel.Informative, ELogColour.Green, " ......... generating hashes for shared cache entries" );
				DirectoryInfo CacheFolderDirectory = new DirectoryInfo( CacheFolder );
				foreach( FileInfo NextFile in CacheFolderDirectory.GetFiles() )
				{
					Log( EVerbosityLevel.Verbose, ELogColour.Green, " ............ creating hash for " + NextFile.Name );

					// Create the hash value and add it to the set
					ChannelHashValues.Add( NextFile.Name, ComputeHash( File.ReadAllBytes( NextFile.FullName ) ) );
				}
#endif // STARTUP_GENERATE_HASHES

#if STARTUP_VERIFY_HASHES
				foreach( FileInfo NextFile in CacheFolderDirectory.GetFiles() )
				{
					Log( EVerbosityLevel.Informative, ELogColour.Green, " ............ verifying hash for " + NextFile.Name );

					// Create the hash value
					byte[] HashValue = ComputeHash( File.ReadAllBytes( NextFile.FullName ) );

					// Look it up in the set
					byte[] ExistingHash;
					if( ChannelHashValues.TryGetValue( NextFile.Name, out ExistingHash ) )
					{
						for( Int32 Index = 0; Index < ExistingHash.Length; Index++ )
						{
							if( ExistingHash[Index] != HashValue[Index] )
							{
								Log( EVerbosityLevel.Informative, ELogColour.Green, " ............     hash mismatch!" );
								break;
							}
						}
					}
				}
#endif // STARTUP_VERIFY_HASHES
			}
			catch( Exception Ex )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Red, " ......... Failed to initialize cache: " + Ex.Message );
				return false;
			}
			return true;
		}

		private Int32 AddChannel_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "AddChannel_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			// First validate the connection handle
            if( Connections.ContainsKey( ConnectionHandle ) )
            {
				// Unpack the input parameters
				string FullPath = InParameters["FullPath"] as string;
				string ChannelName = InParameters["ChannelName"] as string;

                // If the source file exists, try to move it into the cache
                if( File.Exists( FullPath ) )
                {
                    try
                    {
                        string DestinationChannelName = Path.Combine( AgentApplication.Options.CacheFolder, ChannelName );

                        // Make sure the file is writable
                        FileInfo ChannelInfo = new FileInfo( DestinationChannelName );
                        if( ChannelInfo.Exists )
                        {
                            ChannelInfo.IsReadOnly = false;
                            ChannelInfo.Delete();
                        }

						// Remove the channel's entry from the hash table, if one is there
						if( ChannelHashValues.Remove( ChannelName ) )
						{
							Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... removed hash for " + ChannelName );
						}

                        // Before we try to copy anything into it, make sure the cache directory is there
                        ErrorCode = EnsureFolderExists( AgentApplication.Options.CacheFolder );

                        if( ErrorCode >= 0 )
                        {
							// Add the new entry into the channel hash table
							if( ChannelHashValues.Add( ChannelName, ComputeHash( File.ReadAllBytes( FullPath ) ) ) )
							{
								Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... (1) created hash for " + ChannelName );
							}
							else
							{
								Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... (1) discovered duplicate hash for " + ChannelName );
							}

                            // Copy the file into the cache
                            File.Copy( FullPath, DestinationChannelName );

                            ChannelInfo = new FileInfo( DestinationChannelName );
                            ChannelInfo.IsReadOnly = false;

                            ErrorCode = Constants.SUCCESS;
                        }
                    }
                    catch( Exception Ex )
                    {
                        Log( EVerbosityLevel.Informative, ELogColour.Red, "[AddChannel] Failed to add \"" + FullPath + "\"" );
                        Log( EVerbosityLevel.Verbose, ELogColour.Red, "[AddChannel] Exception: " + Ex.ToString() );
                        ErrorCode = Constants.ERROR_EXCEPTION;
                    }
                }
                else
                {
                    ErrorCode = Constants.ERROR_FILE_FOUND_NOT;
                }
            }
            else
            {
				// Not a valid connection, return error
                ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

            StopTiming();
			return( ErrorCode );
		}

		private Int32 TestChannel_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "TestChannel_1_0-Internal", true );

            Int32 ErrorCode = Constants.INVALID;
            
            // First validate the connection handle
			if( Connections.ContainsKey( ConnectionHandle ) )
			{
				// Unpack the input parameters
				string ChannelName = InParameters["ChannelName"] as string;
				
				string FullChannelName = Path.Combine( AgentApplication.Options.CacheFolder, ChannelName );
                if( File.Exists( FullChannelName ) )
                {
					// Ensure we have a valid hash entry for this channel
					if( ChannelHashValues.Add( ChannelName, ComputeHash( File.ReadAllBytes( FullChannelName ) ) ) )
					{
						// Add the entry into the channel hash table
						Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... (2) created hash for " + ChannelName );
					}
					else
					{
						// TODO: Add an option for fully validated cache usage and, when
						// set, add code here to regenerate the hash and compare it here
					}
                    ErrorCode = Constants.SUCCESS;
                }
                else
                {
					// Channel not found, make sure we don't have an entry for it in the hash set
					if( ChannelHashValues.Remove( ChannelName ) )
					{
						Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... removed hash for " + ChannelName );
					}
					ErrorCode = Constants.ERROR_FILE_FOUND_NOT;
                }
            }
            else
            {
                // Not a valid connection, return error
                ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
            }

            StopTiming();
			return ( ErrorCode );
		}

		// We'll use a random value for the handle for better distribution
		// and lookup performance in the dictionary
		private Object SetChannelHandleAndAddLock = new Object();
		private Random SetChannelHandleAndAddGenerator = new Random();
		private Int32 SetChannelHandleAndAdd( Connection ConnectionThatOwnsChannel, Channel NewChannel )
		{
			Int32 NewHandleValue;
			lock( SetChannelHandleAndAddLock )
			{
				// Generate a new random value for the handle
				do
				{
					NewHandleValue = SetChannelHandleAndAddGenerator.Next();
				}
				// Keep generating new values until we find one not already in use
				while( ConnectionThatOwnsChannel.OpenChannels.ContainsKey( NewHandleValue ) );

				// Now set the new handle value and add to the set
				NewChannel.SetHandle( NewHandleValue );
				ConnectionThatOwnsChannel.OpenChannels.Add( NewHandleValue, NewChannel );
			}
			return NewHandleValue;
		}

		private Int32 OpenChannel_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
            StartTiming( "OpenChannel_1_0-Internal", true );

            Int32 ErrorCode = Constants.INVALID;

            // First validate the connection handle
			Connection ConnectionThatOwnsChannel;
            if( Connections.TryGetValue( ConnectionHandle, out ConnectionThatOwnsChannel ) )
            {
				// Unpack the input parameters
				string ChannelName = InParameters["ChannelName"] as string;
				EChannelFlags ChannelFlags = ( EChannelFlags )InParameters["ChannelFlags"];

				Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[OpenChannel] Opening Channel: " + ChannelName );

				// Whether the cache request is a hit
				bool CacheIsHit = true;

				// Determine the proper path for the file and do any necessary preparing
				// depending on the type of channel that's being requested
				try
				{
					ErrorCode = EnsureFolderExists( AgentApplication.Options.CacheFolder );
					if( ErrorCode >= 0 )
					{
						if( ( ChannelFlags & EChannelFlags.TYPE_PERSISTENT ) != 0 )
						{
							string AgentStagingArea = Path.Combine( AgentApplication.Options.CacheFolder, "AgentStagingArea" );
							string StagingAreaChannelName = Path.Combine( AgentStagingArea, ChannelName );
							string FullyCachedChannelName = Path.Combine( AgentApplication.Options.CacheFolder, ChannelName );

							// A channel is opened for writing in the staging area and moved to the cache on Close
							if( ( ChannelFlags & EChannelFlags.ACCESS_WRITE ) != 0 )
							{
								// Ensure the necessary folder exists
								ErrorCode = EnsureFolderExists( AgentStagingArea );
								if( ErrorCode >= 0 )
								{
									// Delete the file if it exists in the staging area
									FileInfo StagingAreaChannel = new FileInfo( StagingAreaChannelName );
									if( StagingAreaChannel.Exists )
									{
										StagingAreaChannel.IsReadOnly = false;
										StagingAreaChannel.Delete();
									}

									// Delete the file if it exists in the main cache area
									FileInfo FullyCachedChannel = new FileInfo( FullyCachedChannelName );
									if( FullyCachedChannel.Exists )
									{
										FullyCachedChannel.IsReadOnly = false;
										FullyCachedChannel.Delete();
									}

									// Remove the channel's entry from the hash table, if one is there
									if( ChannelHashValues.Remove( ChannelName ) )
									{
										Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... removed hash for " + ChannelName );
									}

									// This channel is now safe to open
									// Create a new channel object and add it to the set of open channels
									Channel NewChannel = new Channel( ChannelName, StagingAreaChannelName, ChannelFlags );
									ErrorCode = SetChannelHandleAndAdd( ConnectionThatOwnsChannel, NewChannel );
								}
							}
							// A channel opened for reading is opened directly in the cache
							else if( ( ChannelFlags & EChannelFlags.ACCESS_READ ) != 0 )
							{
								// If the file doesn't exist, see if we can recover
								if( !File.Exists( FullyCachedChannelName ) )
								{
									if( !File.Exists( StagingAreaChannelName ) )
									{
										// If the file doesn't exist, there are a couple cases we need to
										// check before we call this a failure. If this connection is a Job
										// running for a remote Agent, the channel is an implicit dependency
										// and needs to be requested from the remote Agent.
										if( ( ConnectionThatOwnsChannel.Parent != null ) &&
											( ConnectionThatOwnsChannel.Parent is RemoteConnection ) )
										{
											// Request the missing channel via the remote parent connection
											CacheIsHit = false;
											RemoteConnection RemoteParentConnection = ConnectionThatOwnsChannel.Parent as RemoteConnection;
											if( !PullChannel( RemoteParentConnection, ChannelName, null ) )
											{
												// If the pull failed, set the correct error based on the state of the remote
												if( RemoteParentConnection.Interface.IsAlive() )
												{
													ErrorCode = Constants.ERROR_CHANNEL_NOT_FOUND;
												}
												else
												{
													ErrorCode = Constants.ERROR_CONNECTION_DISCONNECTED;
												}
											}
										}
									}
									else
									{
										// This suggests that the file is still being transferred
										// and the caller should wait and try the request again
										ErrorCode = Constants.ERROR_CHANNEL_NOT_READY;
									}
								}
								// If the file does exist, we still may need to validate it with the
								// remote Instigator using the hash value we have for the channel
								else if( ( ConnectionThatOwnsChannel.Parent != null ) &&
										 ( ConnectionThatOwnsChannel.Parent is RemoteConnection ) )
								{
									RemoteConnection RemoteParentConnection = ConnectionThatOwnsChannel.Parent as RemoteConnection;

									byte[] LocalHashValue;
									bool ChannelHashIsValid = false;
									if( !ChannelHashValues.TryGetValue( ChannelName, out LocalHashValue ) )
									{
										LocalHashValue = ComputeHash( File.ReadAllBytes( FullyCachedChannelName ) );
										if( ChannelHashValues.Add( ChannelName, LocalHashValue ) )
										{
											Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... (3) created hash for " + ChannelName );
										}
									}

									if( LocalHashValue.Length > 0 )
									{
										Int32 ParentConnectionHandle = RemoteParentConnection.Handle;
										ChannelHashIsValid = RemoteParentConnection.Interface.ValidateChannel( ParentConnectionHandle, ChannelName, LocalHashValue );
									}
									else
									{
										Log( EVerbosityLevel.Informative, ELogColour.Orange, "[OpenChannel] Error: missing hash table entry for existing channel, pulling " + ChannelName );
									}
									// If we fail the hash comparison test, we need to pull the file
									if( !ChannelHashIsValid )
									{
										Log( EVerbosityLevel.Informative, ELogColour.Orange, "[OpenChannel] Remote hash comparison failed, re-pulling channel " + ChannelName );
										CacheIsHit = false;

										// Before pulling the channel, delete what we've got on disk, in case the pull fails
										FileInfo FullyCachedChannel = new FileInfo( FullyCachedChannelName );
										FullyCachedChannel.IsReadOnly = false;
										FullyCachedChannel.Delete();

										// Remove the channel's entry from the hash table, if one is there
										if( ChannelHashValues.Remove( ChannelName ) )
										{
											Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... removed hash for " + ChannelName );
										}
			
										if( !PullChannel( RemoteParentConnection, ChannelName, null ) )
										{
											// If the pull failed, set the correct error based on the state of the remote
											if( RemoteParentConnection.Interface.IsAlive() )
											{
												ErrorCode = Constants.ERROR_CHANNEL_NOT_FOUND;
											}
											else
											{
												ErrorCode = Constants.ERROR_CONNECTION_DISCONNECTED;
											}
										}
									}
								}

								// Open the file if it exists
								if( File.Exists( FullyCachedChannelName ) )
								{
									// Ensure we have a valid hash entry for this channel
									if( ChannelHashValues.Add( ChannelName, ComputeHash( File.ReadAllBytes( FullyCachedChannelName ) ) ) )
									{
										// Add the entry into the channel hash table
										Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... (4) created hash for " + ChannelName );
									}
									else
									{
										// TODO: Add an option for fully validated cache usage and, when
										// set, add code here to regenerate the hash and compare it here
									}

									// This channel is now safe to open
									// Create a new channel object and add it to the set of open channels
									Channel NewChannel = new Channel( ChannelName, FullyCachedChannelName, ChannelFlags );
									ErrorCode = SetChannelHandleAndAdd( ConnectionThatOwnsChannel, NewChannel );
								}
								else
								{
									// If the file does not exist, make sure we don't have a stale hash for it
									if( ChannelHashValues.Remove( ChannelName ) )
									{
										Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... removed hash for " + ChannelName );
									}
								}
							}
						}
						else if( ( ChannelFlags & EChannelFlags.TYPE_JOB_ONLY ) != 0 )
						{
							AgentGuid JobGuid;
							// If there's a Job associated with this connection, use its GUID
							if( ConnectionThatOwnsChannel.Job != null )
							{
								JobGuid = ConnectionThatOwnsChannel.Job.JobGuid;
							}
							else
							{
								// If there's no Job associated with this connection at this point, provide
								// a default GUID for one for debugging access to the agent cache
								JobGuid = DebuggingJobGuid;
							}

							string AllJobsFolder = Path.Combine( AgentApplication.Options.CacheFolder, "Jobs" );
							string ThisJobFolder = Path.Combine( AllJobsFolder, "Job-" + JobGuid.ToString() );

							// Be sure that the folders we need are there (should have all been created in OpenJob)
							if( Directory.Exists( ThisJobFolder ) )
							{
								string FullJobChannelName = Path.Combine( ThisJobFolder, ChannelName );

								// A channel opened for writing is opened directly in the Jobs folder
								if( ( ChannelFlags & EChannelFlags.ACCESS_WRITE ) != 0 )
								{
									// Delete the file if it already exists
									FileInfo FullChannel = new FileInfo( FullJobChannelName );
									if( FullChannel.Exists )
									{
										FullChannel.IsReadOnly = false;
										FullChannel.Delete();
									}

									// This channel is now safe to open
									// Create a new channel object and add it to the set of open channels
									Channel NewJobChannel = new JobChannel( JobGuid, ChannelName, FullJobChannelName, ChannelFlags );
									ErrorCode = SetChannelHandleAndAdd( ConnectionThatOwnsChannel, NewJobChannel );
								}
								// A channel opened for reading is opened directly from the Jobs folder
								else if( ( ChannelFlags & EChannelFlags.ACCESS_READ ) != 0 )
								{
									// If the file doesn't exist, see if we can recover
									if( !File.Exists( FullJobChannelName ) )
									{
										// If the file doesn't exist, there are a couple cases we need to
										// check before we call this a failure. If this connection is a Job
										// running for a remote Agent, the channel is an implicit dependency
										// and needs to be requested from the remote Agent.
										if( ( ConnectionThatOwnsChannel.Parent != null ) &&
											( ConnectionThatOwnsChannel.Parent is RemoteConnection ) )
										{
											// Request the missing channel via the remote parent connection
											CacheIsHit = false;
											RemoteConnection RemoteParentConnection = ConnectionThatOwnsChannel.Parent as RemoteConnection;
											if( !PullChannel( RemoteParentConnection, ChannelName, JobGuid ) )
											{
												// If the pull failed, set the correct error based on the state of the remote
												if( RemoteParentConnection.Interface.IsAlive() )
												{
													ErrorCode = Constants.ERROR_CHANNEL_NOT_FOUND;
												}
												else
												{
													ErrorCode = Constants.ERROR_CONNECTION_DISCONNECTED;
												}
											}
										}
									}

									// Open the file if it exists
									if( File.Exists( FullJobChannelName ) )
									{
										// This channel is now safe to open
										// Create a new channel object and add it to the set of open channels
										Channel NewJobChannel = new JobChannel( JobGuid, ChannelName, FullJobChannelName, ChannelFlags );
										ErrorCode = SetChannelHandleAndAdd( ConnectionThatOwnsChannel, NewJobChannel );
									}
									else
									{
										// File doesn't exist, error
										ErrorCode = Constants.ERROR_CHANNEL_NOT_FOUND;
									}
								}
							}
						}
					}
				}
				catch( Exception Ex )
				{
					Log( EVerbosityLevel.Informative, ELogColour.Red, "OpenChannel failed: " + Ex.ToString() );
					ErrorCode = Constants.ERROR_EXCEPTION;
				}

				// Register the cache request stats if the channel is being opened for READ
				if( ( ChannelFlags & EChannelFlags.ACCESS_READ ) != 0 )
				{
					ConnectionThatOwnsChannel.CacheRequests++;
					// Only register the hit/miss data if the channel is opened
					if( ErrorCode > 0 )
					{
						ConnectionThatOwnsChannel.CacheHits += CacheIsHit ? 1 : 0;
						ConnectionThatOwnsChannel.CacheMisses += CacheIsHit ? 0 : 1;
					}
					if( ConnectionThatOwnsChannel.Job != null )
					{
						ConnectionThatOwnsChannel.Job.CacheRequests++;
						// Only register the hit/miss data if the channel is opened
						if( ErrorCode > 0 )
						{
							ConnectionThatOwnsChannel.Job.CacheHits += CacheIsHit ? 1 : 0;
							ConnectionThatOwnsChannel.Job.CacheMisses += CacheIsHit ? 0 : 1;
						}
					}
					if( ConnectionThatOwnsChannel.Parent != null )
					{
						ConnectionThatOwnsChannel.Parent.CacheRequests++;
						// Only register the hit/miss data if the channel is opened
						if( ErrorCode > 0 )
						{
							ConnectionThatOwnsChannel.Parent.CacheHits += CacheIsHit ? 1 : 0;
							ConnectionThatOwnsChannel.Parent.CacheMisses += CacheIsHit ? 0 : 1;
						}
					}
				}
			}
			else
			{
				// Not a valid connection, return error
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ErrorCode;
		}

		private Int32 CloseChannel_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
            StartTiming( "CloseChannel_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			// First validate the connection handle
			Connection ConnectionThatOwnsChannel;
			if( Connections.TryGetValue( ConnectionHandle, out ConnectionThatOwnsChannel ) )
			{
				// Unpack the input parameters
				Int32 ChannelHandle = ( Int32 )InParameters["ChannelHandle"];

				Channel ChannelToClose;
				if( ConnectionThatOwnsChannel.OpenChannels.TryGetValue( ChannelHandle, out ChannelToClose ) )
				{
					if( ConnectionThatOwnsChannel.OpenChannels.Remove( ChannelHandle ) )
					{
						// If the channel was opened for writing, and it's not a Job channel
						// move the closed channel into the persistent cache
						if( ( ChannelToClose.Flags & EChannelFlags.ACCESS_WRITE ) != 0 )
						{
							if( ( ChannelToClose is JobChannel ) == false )
							{
								// Channel successfully closed; move file into cache
                                string AgentStagingArea = Path.Combine( AgentApplication.Options.CacheFolder, "AgentStagingArea" );
								string SrcChannelName = Path.Combine( AgentStagingArea, ChannelToClose.Name );
                                string DstChannelName = Path.Combine( AgentApplication.Options.CacheFolder, ChannelToClose.Name );

								// Always remove the destination file if it already exists
                                FileInfo DstChannel = new FileInfo( DstChannelName );
                                if( DstChannel.Exists )
								{
                                    DstChannel.IsReadOnly = false;
                                    DstChannel.Delete();

									// Remove from the deleted channel from the hash table
									Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... removing hash for " + ChannelToClose.Name );
									ChannelHashValues.Remove( ChannelToClose.Name );
								}

								// Add the new entry into the channel hash table
								if( ChannelHashValues.Add( ChannelToClose.Name, ComputeHash( File.ReadAllBytes( SrcChannelName ) ) ) )
								{
									Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... (5) created hash for " + ChannelToClose.Name );
								}

								// Copy if the paper trail is enabled; Move otherwise
								if( ( ChannelToClose.Flags & EChannelFlags.MISC_ENABLE_PAPER_TRAIL ) == 0 )
								{
									File.Move( SrcChannelName, DstChannelName );
								}
								else
								{
									File.Copy( SrcChannelName, DstChannelName );
								}
							}

							// If this connection's parent is a remote connection, the channel needs
							// to be propagated back to the parent
							if( ( ConnectionThatOwnsChannel.Parent != null ) &&
								( ConnectionThatOwnsChannel.Parent is RemoteConnection ) )
							{
								// Send channel via the remote parent connection
								RemoteConnection RemoteParentConnection = ConnectionThatOwnsChannel.Parent as RemoteConnection;
								
								// If this is a job-specific channel, get the GUID
								AgentGuid JobGuid = null;
								if( ChannelToClose is JobChannel )
								{
									JobChannel JobChannelToClose = ChannelToClose as JobChannel;
									JobGuid = JobChannelToClose.JobGuid;
								}

								PushChannel( RemoteParentConnection,
											 ChannelToClose.Name,
											 JobGuid );
							}
						}

						ErrorCode = Constants.SUCCESS;
					}
				}
				else
				{
					ErrorCode = Constants.ERROR_CHANNEL_NOT_FOUND;
				}
			}
			else
			{
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

            StopTiming();
			return ( ErrorCode );
		}

		/**
		 * Pushes a local channel to the remote agent via the remote connection parameter
		 * by calling SendChannel on the remote agent
		 */
		public bool PushChannel(RemoteConnection Remote, string ChannelName, AgentGuid JobGuid)
		{
			StartTiming("PushChannel-Internal", true);
			bool bChannelTransferred = false;

			// The remote connection's handle can be used for all interaction because
			// it has the same meaning on both ends of the connection
			Int32 ConnectionHandle = Remote.Handle;

			EChannelFlags ChannelFlags = EChannelFlags.ACCESS_READ;
			if (JobGuid == null)
			{
				ChannelFlags |= EChannelFlags.TYPE_PERSISTENT;
			}
			else
			{
				ChannelFlags |= EChannelFlags.TYPE_JOB_ONLY;
			}

			Hashtable OpenInParameters = new Hashtable();
			OpenInParameters["Version"] = ESwarmVersionValue.VER_1_0;
			OpenInParameters["ChannelName"] = ChannelName;
			OpenInParameters["ChannelFlags"] = ChannelFlags;
			Hashtable OpenOutParameters = null;

			Int32 LocalChannelHandle = OpenChannel_1_0(ConnectionHandle, OpenInParameters, ref OpenOutParameters);
			if (LocalChannelHandle >= 0)
			{
				try
				{
					string FullChannelName;
					if (JobGuid == null)
					{
						FullChannelName = Path.Combine(AgentApplication.Options.CacheFolder, ChannelName);
					}
					else
					{
						string AllJobsFolder = Path.Combine(AgentApplication.Options.CacheFolder, "Jobs");
						string ThisJobFolder = Path.Combine(AllJobsFolder, "Job-" + JobGuid.ToString());
						FullChannelName = Path.Combine(ThisJobFolder, ChannelName);
					}

					// Read the entire file into a byte stream
					byte[] ChannelData = File.ReadAllBytes(FullChannelName);

					// Send the entire channel at once
					bChannelTransferred = Remote.Interface.SendChannel(ConnectionHandle, ChannelName, ChannelData, JobGuid);

					// If the channel was transferred, track the number of bytes that actually moved across the network
					if (bChannelTransferred)
					{
						FileInfo ChannelInfo = new FileInfo(FullChannelName);
						Remote.NetworkBytesSent += ChannelInfo.Length;
						if (Remote.Job != null)
						{
							Remote.Job.NetworkBytesSent += ChannelInfo.Length;
						}
						if (Remote.Parent != null)
						{
							Remote.Parent.NetworkBytesSent += ChannelInfo.Length;
						}
					}
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Verbose, ELogColour.Red, "[PushChannel] Exception message: " + Ex.ToString());
				}

				Hashtable CloseInParameters = new Hashtable();
				CloseInParameters["Version"] = ESwarmVersionValue.VER_1_0;
				CloseInParameters["ChannelHandle"] = LocalChannelHandle;
				Hashtable CloseOutParameters = null;

				CloseChannel_1_0(ConnectionHandle, CloseInParameters, ref CloseOutParameters);

				if (bChannelTransferred)
				{
					Log(EVerbosityLevel.Verbose, ELogColour.Green, "[Channel] Successful channel push of " + ChannelName);
				}
				else
				{
					Log(EVerbosityLevel.Verbose, ELogColour.Red, string.Format("[PushChannel] Pushing the channel {0} has failed!", ChannelName));
				}
			}
			else
			{
				Log(EVerbosityLevel.Informative, ELogColour.Red, string.Format("[PushChannel] Cannot open local channel {0}.", ChannelName));
			}

			StopTiming();
			return bChannelTransferred;
		}

		/**
		 * Called by a remote Agent, this provides the data for a new channel being pushed
		 */
		public bool SendChannel( Int32 ConnectionHandle, string ChannelName, byte[] ChannelData, AgentGuid JobGuid )
		{
            StartTiming( "SendChannel-Internal", true );

            bool bSucceeded = false;

			// Validate the calling connection
			Connection ConnectionRequestingChannel;
			if( Connections.TryGetValue( ConnectionHandle, out ConnectionRequestingChannel ) &&
				ConnectionRequestingChannel is RemoteConnection )
			{
				EChannelFlags ChannelFlags = EChannelFlags.ACCESS_WRITE;
				if( JobGuid == null )
				{
					ChannelFlags |= EChannelFlags.TYPE_PERSISTENT;
				}
				else
				{
					ChannelFlags |= EChannelFlags.TYPE_JOB_ONLY;
				}

				Hashtable OpenInParameters = new Hashtable();
				OpenInParameters["Version"] = ESwarmVersionValue.VER_1_0;
				OpenInParameters["ChannelName"] = ChannelName;
				OpenInParameters["ChannelFlags"] = ChannelFlags;
				Hashtable OpenOutParameters = null;

				Int32 LocalChannelHandle = OpenChannel_1_0( ConnectionHandle, OpenInParameters, ref OpenOutParameters );
				if( LocalChannelHandle >= 0 )
				{
					// Set up the file name
					string FullChannelName;
					if( JobGuid == null )
					{
						string AgentStagingArea = Path.Combine( AgentApplication.Options.CacheFolder, "AgentStagingArea" );
						FullChannelName = Path.Combine( AgentStagingArea, ChannelName );
					}
					else
					{
						string AllJobsFolder = Path.Combine( AgentApplication.Options.CacheFolder, "Jobs" );
						string ThisJobFolder = Path.Combine( AllJobsFolder, "Job-" + JobGuid.ToString() );
						FullChannelName = Path.Combine( ThisJobFolder, ChannelName );
					}

					// Open the FileStream and write the data directly in
					FileStream NewChannel = null;
					Int32 NewChannelBytes = 0;

					try
					{
						NewChannel = new FileStream( FullChannelName, FileMode.Create, FileAccess.Write, FileShare.None );
						NewChannelBytes = ChannelData.Length;
						NewChannel.Write( ChannelData, 0, NewChannelBytes );
						NewChannel.Close();
					}
					catch( Exception Ex )
					{
						Log( EVerbosityLevel.Informative, ELogColour.Orange, "[SendChannel] Channel \"" + ChannelName + "\" not transferred because of name collision. Channel with that name already exists" );
						Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[SendChannel] Exception: " + Ex.ToString() );
					}

					// Close the channel now that we're done
					Hashtable CloseInParameters = new Hashtable();
					CloseInParameters["Version"] = ESwarmVersionValue.VER_1_0;
					CloseInParameters["ChannelHandle"] = LocalChannelHandle;
					Hashtable CloseOutParameters = null;

					if( CloseChannel_1_0( ConnectionHandle, CloseInParameters, ref CloseOutParameters ) >= 0 )
					{
						// Success
						bSucceeded = true;

						// Track the number of bytes received over the network
						ConnectionRequestingChannel.NetworkBytesReceived += NewChannelBytes;
						if( ConnectionRequestingChannel.Job != null )
						{
							ConnectionRequestingChannel.Job.NetworkBytesReceived += NewChannelBytes;
						}
						if( ConnectionRequestingChannel.Parent != null )
						{
							ConnectionRequestingChannel.Parent.NetworkBytesReceived += NewChannelBytes;
						}
					}
				}
			}

            StopTiming();
            return bSucceeded;
		}

		/**
		 * Pulls a remote channel from the remote agent via the remote connection parameter.
		 * 
		 * @param Remote Remote connection.
		 * @param ChannelName The name of the file to pull.
		 * @param JobGuid A guid of the job.
		 * @param RetriesOnFailure How many times should the pull fail until return a failure.
		 * 
		 * @returns True on success. False otherwise.
		 */
		public bool PullChannel(RemoteConnection Remote, string ChannelName, AgentGuid JobGuid, int RetriesOnFailure = int.MaxValue)
		{
			StartTiming("PullChannel-Internal", true);

			// The remote connection's handle can be used for all interaction because
			// it has the same meaning on both ends of the connection
			Int32 ConnectionHandle = Remote.Handle;

			// Request the file, which will push it back, and keep doing so until we get it
			// or until the connection is dead, which ever comes first
			bool bChannelTransferred = false;
			int TryId = 0;
			while ((Remote.Interface.IsAlive()) &&
				   (bChannelTransferred == false) &&
				   (TryId < RetriesOnFailure))
			{
				try
				{
					bChannelTransferred = Remote.Interface.RequestChannel(ConnectionHandle, ChannelName, JobGuid);
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Verbose, ELogColour.Red, "[PullChannel] Exception message: " + Ex.ToString());
				}

				if(!bChannelTransferred)
				{
					Log(EVerbosityLevel.Verbose, ELogColour.Red, string.Format("[PullChannel] Pulling the channel {0} has failed! Retry {1} of {2}.", ChannelName, TryId + 1, RetriesOnFailure));
				}

				++TryId;
			}

			StopTiming();
			return bChannelTransferred;
		}

		/**
		 * Called by a remote Agent, this requests that a specified file is pushed back
		 */
		public bool RequestChannel( Int32 ConnectionHandle, string ChannelName, AgentGuid JobGuid )
		{
			// Validate the calling connection
			Connection ConnectionRequestingChannel;
			if( Connections.TryGetValue( ConnectionHandle, out ConnectionRequestingChannel ) &&
				ConnectionRequestingChannel is RemoteConnection )
			{
				// Push it back across to the remote Agent
				return PushChannel( ConnectionRequestingChannel as RemoteConnection, ChannelName, JobGuid );
			}
			return false;
		}

		/**
		 * Used to validate a remote channel by providing a remotely generated hash that
		 * we'll use to compare against a locally generated hash - return value is
		 * whether they match
		 */
		public bool ValidateChannel( Int32 ConnectionHandle, string ChannelName, byte[] RemoteChannelHashValue )
		{
			// Validate the calling connection
			Connection ConnectionRequestingChannel;
			if( Connections.TryGetValue( ConnectionHandle, out ConnectionRequestingChannel ) )
			{
				byte[] LocalChannelHashValue;
				lock( ChannelHashValues )
				{
					if( !ChannelHashValues.TryGetValue( ChannelName, out LocalChannelHashValue ) )
					{
						string FullyCachedChannelName = Path.Combine( AgentApplication.Options.CacheFolder, ChannelName );
						if( File.Exists( FullyCachedChannelName ) )
						{
							LocalChannelHashValue = ComputeHash( File.ReadAllBytes( FullyCachedChannelName ) );
							if( ChannelHashValues.Add( ChannelName, LocalChannelHashValue ) )
							{
								Log( EVerbosityLevel.Verbose, ELogColour.Green, " ......... (6) created hash for " + ChannelName );
							}
						}
					}
				}

				if( LocalChannelHashValue.Length > 0 )
				{
					// Compare the length as an easy out
					if( RemoteChannelHashValue.Length == LocalChannelHashValue.Length )
					{
						// Now, compare each byte in the hash
						for( Int32 Index = 0; Index < LocalChannelHashValue.Length; Index++ )
						{
							if( LocalChannelHashValue[Index] != RemoteChannelHashValue[Index] )
							{
								//Log( EVerbosityLevel.Informative, ELogColour.Orange, String.Format( "[ValidateChannel] Hash value mismatch for {0}", ChannelName ) );
								return false;
							}
						}
						// If we make it here, everything matches
						return true;
					}
					else
					{
						Log( EVerbosityLevel.Informative, ELogColour.Orange, String.Format( "[ValidateChannel] Remote channel hash is not the same length as the local one: {0} != {1}", RemoteChannelHashValue.Length, LocalChannelHashValue.Length ) );
					}
				}
				else
				{
					Log( EVerbosityLevel.Informative, ELogColour.Orange, String.Format( "[ValidateChannel] Remote channel name not found in local hash table: {0}", ChannelName ) );
				}
			}
			else
			{
				Log( EVerbosityLevel.Informative, ELogColour.Red, String.Format( "[ValidateChannel] Connection not valid: {0:X8}", ConnectionHandle ) );
			}
			return false;
		}

		/*
		 * Gets the location of the cache
		 */
		public string GetCacheLocation()
		{
            string CacheFolder = AgentApplication.Options.CacheFolder;
            try
            {
                if( EnsureFolderExists( CacheFolder ) < 0 )
                {
                    CacheFolder = "";
                }
            }
            catch( System.Exception )
            {
                CacheFolder = "";
            }

            if( CacheFolder.Length == 0 )
            {
                System.Windows.Forms.MessageBox.Show( "Could not create cache folder '" + AgentApplication.Options.CacheFolder + "'\nPlease update 'SwarmAgent.exe.config' in your binaries folder.",
                                                      "Fatal Error", 
                                                      System.Windows.Forms.MessageBoxButtons.OK, 
                                                      System.Windows.Forms.MessageBoxIcon.Error );

                Log( EVerbosityLevel.Critical, ELogColour.Red, "ERROR: Could not create cache folder '" + CacheFolder + "'\nPlease update the SwarmAgent.exe.config file in your binaries folder." );
            }

            return ( CacheFolder );
		}

		public void RequestCacheRelocation()
		{
			if( Connections.Count > 0 )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Green, "[RequestCacheRelocation] Request delayed because of active, open connections" );
				Log( EVerbosityLevel.Informative, ELogColour.Green, "[RequestCacheRelocation] Relocation will happen after all connections are closed" );
			}
			CacheRelocationRequested = true;
		}

		public void RequestCacheClear()
		{
			if( Connections.Count == 0 )
			{
				CacheClearRequested = true;
			}
			else
			{
				Log( EVerbosityLevel.Informative, ELogColour.Green, "[RequestCacheClear] Request ignored because of active, open connections" );
			}
		}

		public void RequestCacheValidate()
		{
			if( Connections.Count == 0 )
			{
				CacheValidateRequested = true;
			}
			else
			{
				Log( EVerbosityLevel.Informative, ELogColour.Green, "[RequestCacheValidate] Request ignored because of active, open connections" );
			}
		}

		public void CacheAgingThreadProc()
		{
			try
			{
				DateTime OldestDateToKeepUtc;
				string JobsFolder = Path.Combine( AgentApplication.Options.CacheFolder, "Jobs" );
				CleanupJobDirectory( JobsFolder, AgentApplication.Options.MaximumJobsToKeep, out OldestDateToKeepUtc );

				string LogsFolder = Path.Combine( AgentApplication.Options.CacheFolder, "Logs" );
				CleanupLogDirectory( LogsFolder, OldestDateToKeepUtc );

				Int64 MaximumCacheSize = AgentApplication.Options.MaximumCacheSize * ( Int64 )( 1 << 30 );
				Int64 JobsFolderSize = SizeOfDirectory( JobsFolder );
				Int64 LogsFolderSize = SizeOfDirectory( LogsFolder );

				Int64 RemainingSizeToKeep = MaximumCacheSize - JobsFolderSize - LogsFolderSize;

				// For the main cache, sort by last accessed time, and delete everything
				// after we get to the specified size
				DirectoryInfo CacheFolderDirectory = new DirectoryInfo( AgentApplication.Options.CacheFolder );
				FileInfo[] MainCacheFiles = CacheFolderDirectory.GetFiles();
				Array.Sort( MainCacheFiles, new FileAccessTimestampComparer() );

				Log( EVerbosityLevel.Verbose, ELogColour.Green, "[MaintainCache] Cleaning up main cache directory" );
				Int32 FilesDeletedCount = 0;
				foreach( FileInfo NextCacheFile in MainCacheFiles )
				{
					if( RemainingSizeToKeep < 0 )
					{
						// Delete the file
						try
						{
							if( FilesDeletedCount == 0 )
							{
								// If this is the first file deleted, print a warning if
								// it's newer than the oldest job we kept
								if( NextCacheFile.LastAccessTimeUtc >= OldestDateToKeepUtc )
								{
									Log( EVerbosityLevel.Informative, ELogColour.Orange, "[MaintainCache] Deleting a cache entry that is newer than the oldest job kept" );
									Log( EVerbosityLevel.Informative, ELogColour.Orange, "[MaintainCache] Consider increasing the size of the cache (in Settings)" );
								}
							}
							File.Delete( NextCacheFile.FullName );
							FilesDeletedCount++;
						}
						catch( Exception Ex )
						{
							Log( EVerbosityLevel.Verbose, ELogColour.Red, "[MaintainCache] Error cleaning up " + NextCacheFile.FullName );
							Log( EVerbosityLevel.Verbose, ELogColour.Red, "[MaintainCache] Exception: " + Ex.Message );
						}
					}
					else
					{
						RemainingSizeToKeep -= NextCacheFile.Length;
					}
				}
			}
			catch( Exception Ex )
			{
				// Just catch and do nothing with it for now
				Log( EVerbosityLevel.ExtraVerbose, ELogColour.Red, "[MaintainCache] Exception: " + Ex.Message );
			}
		}

		public void MaintainCache()
		{
			// If we're not initialied yet, do nothing
			if( !Initialized.WaitOne(0) )
			{
				return;
			}

			// Ensure no new connection can come online while we're doing cache work
			lock( Connections )
			{
				// If there are no active connections, then we can do cache upkeep
				if( Connections.Count == 0 )
				{
					// Relocate the cache if requested
					if( CacheRelocationRequested )
					{
						// Simply call init cache to re-read location and recreate what we need
						InitCache();
						CacheRelocationRequested = false;
					}
					// If a full cache clearing out has been requested
					if( CacheClearRequested )
					{
						Log( EVerbosityLevel.Informative, ELogColour.Green, "[CleanCache] Clearing out the cache by user request" );

						// Delete everything in the persistent cache
						string[] PersistentCacheContents = Directory.GetFiles( AgentApplication.Options.CacheFolder );
						bool CacheCleared = true;
						foreach( string PersistentCacheEntry in PersistentCacheContents )
						{
							try
							{
								File.Delete( PersistentCacheEntry );
							}
							catch ( Exception Ex )
							{
								Log( EVerbosityLevel.Informative, ELogColour.Orange, "[CleanCache] Failed to delete the file " + PersistentCacheEntry + ", it may be in use..." );
								Log( EVerbosityLevel.Informative, ELogColour.Orange, "[CleanCache] Exception was: " + Ex.Message.TrimEnd( '\n' ) );
								CacheCleared = false;
							}
						}
						if( CacheCleared )
						{
							Log( EVerbosityLevel.Informative, ELogColour.Green, "[CleanCache] Cache successfully cleared" );
						}
						else
						{
							Log( EVerbosityLevel.Informative, ELogColour.Green, "[CleanCache] Cache not cleared, some files were not deleted, but the Agent will pretend they're gone" );
						}

						// Clear out the hash set
						ChannelHashValues.Clear();

						// Reset the requesting boolean
						CacheClearRequested = false;
					}
					// If a cache validation has been requested, perform one now
					else if( CacheValidateRequested )
					{
						// Iterate through the entire cache and compare newly generated values 
						Log( EVerbosityLevel.Informative, ELogColour.Green, "[MaintainCache] Validating all cache entries and hashes" );
						DirectoryInfo CacheFolderDirectory = new DirectoryInfo( AgentApplication.Options.CacheFolder );

						// Walk through all existing files and compare new hashes with their existing hashes
						ReaderWriterDictionary<string, byte[]> ValidatedChannelHashValues = new ReaderWriterDictionary<string, byte[]>();

						bool AllFilesUpdated = true;
						foreach( FileInfo NextFile in CacheFolderDirectory.GetFiles() )
						{
							// Each iteration, check to see if a local connection request has come in
							if( LocalConnectionRequestWaiting.WaitOne( 0 ) )
							{
								// If there is, then break out of this work loop
								Log( EVerbosityLevel.Informative, ELogColour.Green, "[MaintainCache] Canceling validation due to incoming local connection" );
								AllFilesUpdated = false;
								break;
							}

							Log( EVerbosityLevel.Informative, ELogColour.Green, " ............ validating hash for " + NextFile.Name );

							// Create the hash value and add it to the new set
							byte[] HashValue = ComputeHash( File.ReadAllBytes( NextFile.FullName ) );
							ValidatedChannelHashValues.Add( NextFile.Name, HashValue );

							// Look it up in the old set
							byte[] ExistingHash;
							if( ChannelHashValues.TryGetValue( NextFile.Name, out ExistingHash ) )
							{
								// Compare the length as an easy out
								bool HashMatches = false;
								if( ExistingHash.Length == HashValue.Length )
								{
									// Compare each entry in the hash value
									HashMatches = true;
									for( Int32 Index = 0; Index < ExistingHash.Length; Index++ )
									{
										if( ExistingHash[Index] != HashValue[Index] )
										{
											HashMatches = false;
											break;
										}
									}
								}
								// Report whether it matched
								if( !HashMatches )
								{
									Log( EVerbosityLevel.Informative, ELogColour.Orange, " ............     hash mismatch, updated" );
								}
								else
								{
									Log( EVerbosityLevel.Verbose, ELogColour.Green, " ............     hash validated" );
								}
								// Remove the entry from the old set
								ChannelHashValues.Remove( NextFile.Name );
							}
							else
							{
								Log( EVerbosityLevel.Informative, ELogColour.Orange, " ............     hash missing, updated" );
							}
						}

						if( AllFilesUpdated )
						{
							// Report all remaining hashes that do not correspond to existing files
							if( ChannelHashValues.Count > 0 )
							{
								Log( EVerbosityLevel.Informative, ELogColour.Orange, " ............ Found orphaned hashes for the following missing channels:" );
								foreach( string OrphanChannelName in ChannelHashValues.Keys )
								{
									Log( EVerbosityLevel.Informative, ELogColour.Orange, " ............     " + OrphanChannelName );
								}
								// Clear out the old set
								ChannelHashValues.Clear();
							}

							// Set the new, validated set
							ChannelHashValues = ValidatedChannelHashValues;

							Log( EVerbosityLevel.Informative, ELogColour.Green, "[MaintainCache] Cache hashes validated and up to date" );
						}

						// Reset the requesting boolean
						CacheValidateRequested = false;
					}
					// Otherwise, if it's been long enough since we last ran the clean up routine
                    else if (DateTime.UtcNow > NextCleanUpCacheTime)
					{
						Log( EVerbosityLevel.Verbose, ELogColour.Green, "[MaintainCache] Performing cache clean up" );

						Thread CacheAgingThread = new Thread( new ThreadStart( CacheAgingThreadProc ) );
						CacheAgingThread.Name = "Cache Aging Thread";
						CacheAgingThread.Start();
						
						bool KeepRunning = true;
						while( KeepRunning )
						{
							// If a local connection request comes in, quit immediately
							if( LocalConnectionRequestWaiting.WaitOne( 0 ) )
							{
								CacheAgingThread.Abort();
								KeepRunning = false;
							}
							else if( CacheAgingThread.Join( 100 ) )
							{
								// If the thread is done, quit
								KeepRunning = false;
							}
						}

						// Set the next clean up time
                        NextCleanUpCacheTime = DateTime.UtcNow + TimeSpan.FromMinutes(10);

						Log( EVerbosityLevel.Verbose, ELogColour.Green, "[MaintainCache] Cache clean up complete" );
					}
				}
			}
		}
	}
}

