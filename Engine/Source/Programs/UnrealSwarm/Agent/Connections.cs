// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
#if !__MonoCS__
using System.Deployment.Application;
#endif
using System.Diagnostics;
using System.IO;
using System.Management;
using System.Net;
using System.Net.NetworkInformation;
using System.Reflection;
using System.Runtime.Remoting;
using System.Runtime.Remoting.Channels;
using System.Runtime.Remoting.Channels.Tcp;
using System.Threading;
using System.Linq;

using AgentInterface;
using SwarmCoordinatorInterface;
using SwarmCommonUtils;
using System.Net.Sockets;

namespace Agent
{
	///////////////////////////////////////////////////////////////////////////

	public enum ConnectionState
	{
		UNINITIALIZED,	// Connections start in this state
		CONNECTED,		// Once the connection is set up and ready
		DISCONNECTING,	// Once CloseConnection is called
		DISCONNECTED,	// Once CloseConnection is finished
	}

	/**
	 * The base Connection class that contains any data and methods common to
	 * any connection type, local or remote
	 */
	public class Connection
	{
		/**
		 * The unique handle associated with this connection
		 */
		public Int32 Handle = Constants.INVALID;

		/**
		 * A simple boolean that indicates whether this connection
		 * is actually connected right now
		 */
		public ConnectionState CurrentState = ConnectionState.UNINITIALIZED;

		/**
		 * Any connection can have up to one parent connection. This
		 * parent connection, if there is one, is the connection that
		 * this connection is doing work on behalf of.
		 */
		public Connection Parent = null;

		/**
		 * A connection can have a set of other connections working on
		 * its behalf, executing and completing tasks for jobs that it
		 * needs help finishing. Key is the connection's handle.
		 */
		public UInt32 ChildrenSeen = 0;
		public ReaderWriterDictionary<Int32, LocalConnection> LocalChildren = new ReaderWriterDictionary<Int32, LocalConnection>();
		public UInt32 LocalChildrenSeen = 0;
		public ReaderWriterDictionary<Int32, RemoteConnection> RemoteChildren = new ReaderWriterDictionary<Int32, RemoteConnection>();
		public UInt32 RemoteChildrenSeen = 0;

		/**
		 * All channels opened for this connection
		 */
		public ReaderWriterDictionary<Int32, Channel> OpenChannels = new ReaderWriterDictionary<Int32, Channel>();

		/**
		 * Cache usage statistics for the Connection
		 */
		public Int32 CacheRequests = 0;
		public Int32 CacheMisses = 0;
		public Int32 CacheHits = 0;

		/**
		 * Simple push/pull channel statistics for the Connection
		 */
		public Int64 NetworkBytesSent = 0;
		public Int64 NetworkBytesReceived = 0;

		/**
		 * The Job this connection owns, if any, of which there can be only
		 * one at any time and each Job can have only one owner per Agent
		 */
		public AgentJob Job = null;

		/**
		 * All Tasks assigned to this connection
		 */
		public ReaderWriterDictionary<AgentGuid, AgentTask> RunningTasks = new ReaderWriterDictionary<AgentGuid, AgentTask>();

		/**
		 * A count of open reservations used to send out new tasks as they come in via AddTask
		 */
		public UInt32 ReservationCount = 0;

		/**
		 * Times the connection was opened and closed
		 */
        public DateTime ConnectedTime = DateTime.UtcNow;
		public DateTime DisconnectedTime;

		///////////////////////////////////////////////////////////////////////////

		/**
		 * Standard constructor
		 */
		public Connection( Int32 ConnectionHandle )
		{
			Handle = ConnectionHandle;
		}
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * A Connection class that contains any additional methods and data for local
	 * connections, including: a message queue, lists of available remote agents,
	 * and the process ID of the locally running process on the other end of the
	 * connection.
	 */
	public class LocalConnection : Connection
	{
		/**
		 * The system process id (pid) of the application opening the local
		 * connection. Used, in part, to connect job application connections
		 * with the job owning connections.
		 */
		private Int32 Private_ProcessID = 0;
		public Int32 ProcessID
		{
			get { return Private_ProcessID; }
			set { Private_ProcessID = value; }
		}

		/**
		 * A list of potential remote agents that can help with jobs and a list
		 * of those potential agents we've tried to contact, but have failed and
		 * may want to try contacting again in the future
		 */
		private Queue<AgentInfo> Private_PotentialRemoteAgents = new Queue<AgentInfo>();
		public Queue<AgentInfo> PotentialRemoteAgents
		{
			get { return Private_PotentialRemoteAgents; }
			set { Private_PotentialRemoteAgents = value; }
		}
		private Queue<AgentInfo> Private_UnavailableRemoteAgents = new Queue<AgentInfo>();
		public Queue<AgentInfo> UnavailableRemoteAgents
		{
			get { return Private_UnavailableRemoteAgents; }
			set { Private_UnavailableRemoteAgents = value; }
		}
		public bool RemoteAgentsAbandoned = false;

		/**
		 * The message queue used to send messages back to the application
		 * on the other end of this connection
		 */
		private Queue<AgentMessage> Private_MessageQueue = new Queue<AgentMessage>();
		public Queue<AgentMessage> MessageQueue
		{
			get { return Private_MessageQueue; }
			set { Private_MessageQueue = value; }
		}

		/**
		 * A simple signal set whenever there is a message available in the queue
		 */
		private Semaphore Private_MessageAvailable = new Semaphore( 0, 10000 );
		public Semaphore MessageAvailable
		{
			get { return Private_MessageAvailable; }
			set { Private_MessageAvailable = value; }
		}
		private ReaderWriterLockSlim OverflowCounterLock = new ReaderWriterLockSlim();
		private Int32 OverflowCounter = 0;

		///////////////////////////////////////////////////////////////////////////

		/**
		 * Standard constructor
		 */
		public LocalConnection( Int32 ConnectionHandle,
								Int32 ConnectionProcessID )
			: base( ConnectionHandle )
		{
			ProcessID = ConnectionProcessID;
		}

		public void MessageAvailableSignal()
		{
			// Try to release the semaphore
			try
			{
				MessageAvailable.Release();
			}
			catch( SemaphoreFullException )
			{
				// If it's already full, increment the overflow counter
				OverflowCounterLock.EnterWriteLock();
				OverflowCounter++;
				OverflowCounterLock.ExitWriteLock();
			}
			catch( Exception )
			{
				// Otherwise, the release will be dropped
			}
		}

		public bool MessageAvailableWait( Int32 TimeOut )
		{
			bool Signaled = MessageAvailable.WaitOne( TimeOut );
			if( Signaled )
			{
				// If the semaphore was signaled, check to see if there are
				// any overflowed signals waiting to be applied
				OverflowCounterLock.EnterUpgradeableReadLock();
				try
				{
					if( OverflowCounter > 0 )
					{
						// If there are, try to apply them
						OverflowCounterLock.EnterWriteLock();
						try
						{
							// If we're able to release, then decrement the counter
							MessageAvailable.Release();
							OverflowCounter--;
						}
						catch( Exception )
						{
							// Otherwise, we can't decrement
						}

						OverflowCounterLock.ExitWriteLock();
					}
				}
				finally
				{
					OverflowCounterLock.ExitUpgradeableReadLock();
				}
			}
			return Signaled;
		}
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * A wrapper around the communication object we use for agent-to-agent
	 * API calls and data transfer. This wrapper knows the current state of
	 * the connection and is used to determine if the connection has been
	 * dropped
	 */
	public class RemoteConnectionInterfaceWrapper
	{
		public RemoteConnectionInterfaceWrapper( Int32 NewRemoteAgentHandle, string NewRemoteAgentName, string NewRemoteAgentIPAddress, Agent NewRemoteInterface )
		{
			RemoteAgentName = NewRemoteAgentName;
			RemoteAgentIPAddress = NewRemoteAgentIPAddress;
			RemoteAgentHandle = NewRemoteAgentHandle;
			RemoteInterface = NewRemoteInterface;

			RemoteInterfaceDropped = new ManualResetEvent( false );
			RemoteInterfaceAlive = true;
			RemoteInterfaceMonitorThread = null;
		}

		Object SignalConnectionDroppedLock = new Object();
		public void SignalConnectionDropped()
		{
			lock( SignalConnectionDroppedLock )
			{
				AgentApplication.Log( EVerbosityLevel.Verbose, ELogColour.Green, "[SignalConnectionDropped] Connection dropped for " + RemoteAgentName );
				RemoteInterfaceAlive = false;
				RemoteInterfaceDropped.Set();

				// Once the connection is no longer active, we can stop monitoring
				EndMonitoring();
			}
		}

		delegate bool TestConnectionDelegate( Int32 ConnectionHandle );

		public void BeginMonitoring()
		{
			// Spawn a thread that will simply run a monitoring loop for this connection
			AgentApplication.Log( EVerbosityLevel.Verbose, ELogColour.Green, "[BeginMonitoring] Starting monitoring for " + RemoteAgentName );
			RemoteInterfaceMonitorThread = new Thread( new ThreadStart( RemoteInterfaceMonitorThreadProc ) );
			RemoteInterfaceMonitorThread.Name = "Connection Monitor for " + RemoteAgentName;
			RemoteInterfaceMonitorThread.Start();
		}

		public void EndMonitoring()
		{
			// The monitor thread could have quit by now, but if not, help it out
			if( ( RemoteInterfaceMonitorThread != null ) &&
				( RemoteInterfaceMonitorThread.Join( 0 ) == false ) )
			{
				RemoteInterfaceMonitorThread.Abort();
			}
			RemoteInterfaceMonitorThread = null;
			AgentApplication.Log( EVerbosityLevel.Verbose, ELogColour.Green, "[BeginMonitoring] Monitoring ended for " + RemoteAgentName );
		}

		private void RemoteInterfaceMonitorThreadProc()
		{
			try
			{
				// First, check if we need to update the status
				while( RemoteInterfaceAlive )
				{
					// Start by marking it dead and allowing only success below to mark it alive
					bool NewRemoteInterfaceAlive = false;
					try
					{
						AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[RemoteInterfaceMonitorThreadProc] Pinging " + RemoteAgentName + " at " + RemoteAgentIPAddress + " ..." );

						Ping PingSender = new Ping();
						// Slight modification to the value to allow < 0 be infinite, rather than 0 (Ping.Send differs from Wait* in this way)
						Int32 RemoteAgentTimeout = ( AgentApplication.DeveloperOptions.RemoteAgentTimeout >= 0 ? ( ( AgentApplication.DeveloperOptions.RemoteAgentTimeout * 1000 ) + 1 ) : 0 );
						PingReply Reply = PingSender.Send( RemoteAgentIPAddress, RemoteAgentTimeout );
						if( Reply.Status == IPStatus.Success )
						{
							AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[RemoteInterfaceMonitorThreadProc] Testing connection on " + RemoteAgentName + " ..." );

							// Test the connection and update the next ping time
							TestConnectionDelegate DTestConnection = new TestConnectionDelegate( RemoteInterface.TestConnection );
							IAsyncResult Result = DTestConnection.BeginInvoke( RemoteAgentHandle, null, null );

							// For the determination of whether the connection is alive, use a timeout
							RemoteAgentTimeout = ( AgentApplication.DeveloperOptions.RemoteAgentTimeout >= 0 ? AgentApplication.DeveloperOptions.RemoteAgentTimeout * 1000 : Timeout.Infinite );
							WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped }, RemoteAgentTimeout );
							if( Result.IsCompleted )
							{
								NewRemoteInterfaceAlive = DTestConnection.EndInvoke( Result );
							}
						}
					}
					catch( Exception Ex )
					{
						AgentApplication.Log( EVerbosityLevel.Verbose, ELogColour.Red, "[RemoteInterfaceMonitorThreadProc] Remote connection exception: " + Ex.Message );
					}

					// If we've just dropped the connection, signal it
					if( NewRemoteInterfaceAlive == false )
					{
						AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[RemoteInterfaceMonitorThreadProc] " + RemoteAgentName + " has dropped, cleaning up" );
						SignalConnectionDropped();
					}
					else
					{
						// Otherwise, sleep for a bit before trying again
						AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[RemoteInterfaceMonitorThreadProc] " + RemoteAgentName + " is still alive" );
						Thread.Sleep( 2000 );
					}
				}
			}
			catch( Exception )
			{
			}
			finally
			{
				AgentApplication.Log( EVerbosityLevel.Verbose, ELogColour.Green, "[RemoteInterfaceMonitorThreadProc] Monitor thread exiting " + RemoteAgentName + " ..." );
			}
		}

		public bool IsAlive()
		{
			AgentApplication.Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, "[IsAlive] " + RemoteAgentName + "::IsAlive returns " + RemoteInterfaceAlive.ToString() );
			return RemoteInterfaceAlive;
		}

		/**
		 * Wrapper implementations for the API
		 */
		delegate Int32 OpenRemoteConnectionDelegate( string RequestingAgentName, Int32 ConnectionHandle, ELogFlags LoggingFlags, bool IsAssigned, DateTime AssignedTimestamp );
		public Int32 OpenRemoteConnection( string RequestingAgentName, Int32 ConnectionHandle, ELogFlags LoggingFlags, bool IsAssigned, DateTime AssignedTimestamp )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					OpenRemoteConnectionDelegate DOpenRemoteConnection = new OpenRemoteConnectionDelegate( RemoteInterface.OpenRemoteConnection );
					IAsyncResult Result = DOpenRemoteConnection.BeginInvoke( RequestingAgentName, ConnectionHandle, LoggingFlags, IsAssigned, AssignedTimestamp, null, null );
					// Double the timeout, since this requires round-trip communication
					Int32 RemoteAgentTimeout = ( AgentApplication.DeveloperOptions.RemoteAgentTimeout >= 0 ? 2 * AgentApplication.DeveloperOptions.RemoteAgentTimeout * 1000 : Timeout.Infinite );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped }, RemoteAgentTimeout );
					if( Result.IsCompleted )
					{
						ReturnCode = DOpenRemoteConnection.EndInvoke( Result );
						// If opening the remote connection is successful, start monitoring for drops
						if( ReturnCode >= 0 )
						{
							BeginMonitoring();
						}
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate Int32 ConfirmRemoteConnectionDelegate( Int32 ConnectionHandle );
		public Int32 ConfirmRemoteConnection( Int32 ConnectionHandle )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					ConfirmRemoteConnectionDelegate DConfirmRemoteConnection = new ConfirmRemoteConnectionDelegate( RemoteInterface.ConfirmRemoteConnection );
					IAsyncResult Result = DConfirmRemoteConnection.BeginInvoke( ConnectionHandle, null, null );
					Int32 RemoteAgentTimeout = ( AgentApplication.DeveloperOptions.RemoteAgentTimeout >= 0 ? AgentApplication.DeveloperOptions.RemoteAgentTimeout * 1000 : Timeout.Infinite );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped }, RemoteAgentTimeout );
					if( Result.IsCompleted )
					{
						ReturnCode = DConfirmRemoteConnection.EndInvoke( Result );
						// If confirming the remote connection is successful, start monitoring for drops
						if( ReturnCode >= 0 )
						{
							BeginMonitoring();
						}
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate Int32 CloseConnectionDelegate( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters );
		public Int32 CloseConnection( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					CloseConnectionDelegate DCloseConnection = new CloseConnectionDelegate( RemoteInterface.CloseConnection );
					IAsyncResult Result = DCloseConnection.BeginInvoke( ConnectionHandle, InParameters, ref OutParameters, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DCloseConnection.EndInvoke( ref OutParameters, Result );
					}
				}
				catch( Exception )
				{
				}

				// Always signal this as when the connection is dropped
				SignalConnectionDropped();
			}
			return ReturnCode;
		}

		delegate Int32 SendMessageDelegate( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters );
		public Int32 SendMessage( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					SendMessageDelegate DSendMessage = new SendMessageDelegate( RemoteInterface.SendMessage );
					IAsyncResult Result = DSendMessage.BeginInvoke( ConnectionHandle, InParameters, ref OutParameters, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DSendMessage.EndInvoke( ref OutParameters, Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate bool SendChannelDelegate( Int32 ConnectionHandle, string ChannelName, byte[] ChannelData, AgentGuid JobGuid );
		public bool SendChannel( Int32 ConnectionHandle, string ChannelName, byte[] ChannelData, AgentGuid JobGuid )
		{
			bool ReturnCode = false;
			if( RemoteInterfaceAlive )
			{
				try
				{
					SendChannelDelegate DSendChannel = new SendChannelDelegate( RemoteInterface.SendChannel );
					IAsyncResult Result = DSendChannel.BeginInvoke( ConnectionHandle, ChannelName, ChannelData, JobGuid, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DSendChannel.EndInvoke( Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate bool RequestChannelDelegate( Int32 ConnectionHandle, string ChannelName, AgentGuid JobGuid );
		public bool RequestChannel( Int32 ConnectionHandle, string ChannelName, AgentGuid JobGuid )
		{
			bool ReturnCode = false;
			if( RemoteInterfaceAlive )
			{
				RetryCatch(
					// Try {
					() =>
					{
						RequestChannelDelegate DRequestChannel = new RequestChannelDelegate(RemoteInterface.RequestChannel);
						IAsyncResult Result = DRequestChannel.BeginInvoke(ConnectionHandle, ChannelName, JobGuid, null, null);
						WaitHandle.WaitAny(new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped });
						if (Result.IsCompleted)
						{
							ReturnCode = DRequestChannel.EndInvoke(Result);
						}
					},
					// } Catch (Exception) {
					() =>
					{
						SignalConnectionDropped();
					},
					// }
					5
				);
			}
			return ReturnCode;
		}

		delegate bool ValidateChannelDelegate( Int32 ConnectionHandle, string ChannelName, byte[] ChannelHashValue );
		public bool ValidateChannel( Int32 ConnectionHandle, string ChannelName, byte[] ChannelHashValue )
		{
			bool ReturnCode = false;
			if( RemoteInterfaceAlive )
			{
				try
				{
					ValidateChannelDelegate DValidateChannel = new ValidateChannelDelegate( RemoteInterface.ValidateChannel );
					IAsyncResult Result = DValidateChannel.BeginInvoke( ConnectionHandle, ChannelName, ChannelHashValue, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DValidateChannel.EndInvoke( Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate Int32 OpenJobDelegate( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters );
		public Int32 OpenJob( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					OpenJobDelegate DOpenJob = new OpenJobDelegate( RemoteInterface.OpenJob );
					IAsyncResult Result = DOpenJob.BeginInvoke( ConnectionHandle, InParameters, ref OutParameters, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DOpenJob.EndInvoke( ref OutParameters, Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate Int32 BeginJobSpecificationDelegate( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters );
		public Int32 BeginJobSpecification( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					BeginJobSpecificationDelegate DBeginJobSpecification = new BeginJobSpecificationDelegate( RemoteInterface.BeginJobSpecification );
					IAsyncResult Result = DBeginJobSpecification.BeginInvoke( ConnectionHandle, InParameters, ref OutParameters, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DBeginJobSpecification.EndInvoke( ref OutParameters, Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate Int32 AddTaskDelegate( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters );
		public Int32 AddTask( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					AddTaskDelegate DAddTask = new AddTaskDelegate( RemoteInterface.AddTask );
					IAsyncResult Result = DAddTask.BeginInvoke( ConnectionHandle, InParameters, ref OutParameters, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DAddTask.EndInvoke( ref OutParameters, Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate Int32 EndJobSpecificationDelegate( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters );
		public Int32 EndJobSpecification( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					EndJobSpecificationDelegate DEndJobSpecification = new EndJobSpecificationDelegate( RemoteInterface.EndJobSpecification );
					IAsyncResult Result = DEndJobSpecification.BeginInvoke( ConnectionHandle, InParameters, ref OutParameters, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DEndJobSpecification.EndInvoke( ref OutParameters, Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		delegate Int32 CloseJobDelegate( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters );
		public Int32 CloseJob( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ReturnCode = Constants.ERROR_CONNECTION_DISCONNECTED;
			if( RemoteInterfaceAlive )
			{
				try
				{
					CloseJobDelegate DCloseJob = new CloseJobDelegate( RemoteInterface.CloseJob );
					IAsyncResult Result = DCloseJob.BeginInvoke( ConnectionHandle, InParameters, ref OutParameters, null, null );
					WaitHandle.WaitAny( new WaitHandle[2] { Result.AsyncWaitHandle, RemoteInterfaceDropped } );
					if( Result.IsCompleted )
					{
						ReturnCode = DCloseJob.EndInvoke( ref OutParameters, Result );
					}
				}
				catch( Exception )
				{
					SignalConnectionDropped();
				}
			}
			return ReturnCode;
		}

		private void RetryCatch(Action TryClause, Action CatchClause, int RetriesOnFailure = 5)
		{
			int TryId = 1;
			while(TryId <= RetriesOnFailure)
			{
				try
				{
					TryClause();
					break;
				}
				catch(Exception Ex)
				{
					if(TryId == RetriesOnFailure)
					{
						AgentApplication.Log(EVerbosityLevel.Critical, ELogColour.Red, string.Format("Error (retry {0} of {1}): {2}. Stack trace:\n{3}\n", TryId, RetriesOnFailure, Ex.Message, Ex.StackTrace));
						CatchClause();
					}
					else
					{
						AgentApplication.Log(EVerbosityLevel.Critical, ELogColour.Orange, string.Format("Error (retry {0} of {1}): {2}. Stack trace: {3}.\n", TryId, RetriesOnFailure, Ex.Message, Ex.StackTrace));
					}
				}

				++TryId;
			}
		}

		public string				RemoteAgentName;
		public string				RemoteAgentIPAddress;
		private Int32				RemoteAgentHandle;
		private Agent				RemoteInterface;
		private ManualResetEvent	RemoteInterfaceDropped;
		private bool				RemoteInterfaceAlive;
		private Thread				RemoteInterfaceMonitorThread;
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * A Connection class that contains any additional methods and data for remote
	 * connections, including: an agent interface used to communicate directly to
	 * the remote object
	 */
	public class RemoteConnection : Connection
	{
		/**
		 * The host name of the remote system
		 */
		private AgentInfo Private_Info = null;
		public AgentInfo Info
		{
			get { return Private_Info; }
			set { Private_Info = value; }
		}

		/**
		 * The interface to the remote agent, used for all direct communication
		 */
		private RemoteConnectionInterfaceWrapper Private_Interface = null;
		public RemoteConnectionInterfaceWrapper Interface
		{
			get { return Private_Interface; }
			set { Private_Interface = value; }
		}

		/**
		 * A list of all known aliases for this connection, possibly created when
		 * a remote Job executable send a message to the instigator and we create
		 * an alias for response routing
		 */
		private List<Int32> Private_Aliases = new List<Int32>();
		public List<Int32> Aliases
		{
			get { return Private_Aliases; }
			set { Private_Aliases = value; }
		}

		///////////////////////////////////////////////////////////////////////////

		/**
		 * Standard constructor
		 */
		public RemoteConnection( Int32 ConnectionHandle,
								 AgentInfo NewRemoteAgentInfo,
								 RemoteConnectionInterfaceWrapper NewRemoteInterface )
			: base( ConnectionHandle )
		{
			Info = NewRemoteAgentInfo;
			Interface = NewRemoteInterface;
		}
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of connection management behavior in the Agent
	 */
	public partial class Agent : MarshalByRefObject, IAgentInternalInterface, IAgentInterface
	{
		///////////////////////////////////////////////////////////////////////

		// Interface to the global coordinator and the agent's state (sent to the coordinator)
		private ISwarmCoordinator Coordinator;
		private bool CoordinatorResponding;
		private AgentState CurrentState = AgentState.Available;
		private string WorkingFor = "";

		// Variables used for communicating with the coordinator
		private static DateTime LastCoordinatorPingTime = DateTime.MinValue;
		private static bool AgentIsRestarting = false;

		// Opened and active connections to this agent
		private ReaderWriterDictionary<Int32, Connection> Connections = new ReaderWriterDictionary<Int32, Connection>();
		private ManualResetEvent LocalConnectionRequestWaiting = new ManualResetEvent( false );
		// Connections that are in the process of being established. Created, not confirmed.
		private ReaderWriterDictionary<Int32, RemoteConnection> PendingConnections = new ReaderWriterDictionary<Int32, RemoteConnection>();
		// The timestamp of the last "assigned" connection
		DateTime LastAssignedTime = DateTime.MinValue;

		///////////////////////////////////////////////////////////////////////

		/// <summary>
		/// Validates host name. Currently checks if the name contains only numeric characters
		/// Throws an exception if the name is invalid
		/// </summary>
		public void ValidateHostName(string HostName)
		{
			bool bNumericOnly = true;
			foreach (char C in HostName)
			{
				if (!Char.IsDigit(C))
				{
					bNumericOnly = false;
				}
			}
			if (bNumericOnly)
			{
				string Message = String.Format("Host name {0} contains only numbers and it may not be possible to connect to it", HostName);
				Log(EVerbosityLevel.Informative, ELogColour.Red, " ......... " + Message);
				throw new Exception(Message);
			}
		}

		/**
		 * Initializes the coordinator interface
		 */
		public void InitCoordinator()
		{
			try
			{
				// Get the coordinator interface object
				Log( EVerbosityLevel.Informative, ELogColour.Green, " ......... using SwarmCoordinator on " + AgentApplication.Options.CoordinatorRemotingHost );
#if false
				// Use a local copy of the coordinator, for debugging
				string CoordinatorObjectURL = String.Format( "tcp://127.0.0.1:{0}/SwarmCoordinator",
					Properties.Settings.Default.CoordinatorRemotingPort );
#else
				ValidateHostName(AgentApplication.Options.CoordinatorRemotingHost);

				string CoordinatorObjectURL = String.Format( "tcp://{0}:{1}/SwarmCoordinator",
					AgentApplication.Options.CoordinatorRemotingHost,
					Properties.Settings.Default.CoordinatorRemotingPort );
#endif
				Coordinator = ( ISwarmCoordinator )Activator.GetObject( typeof( ISwarmCoordinator ), CoordinatorObjectURL );
				CoordinatorResponding = true;

				if( AgentApplication.Options.EnableStandaloneMode == false )
				{
					// Update the responding state using the first ping to the coordinator
					CoordinatorResponding = PingCoordinator( true );
				}
			}
			catch( Exception Ex )
			{
				// If we fail for any reason, don't try to talk to the coordinator again
				Log( EVerbosityLevel.ExtraVerbose, ELogColour.Orange, "[InitCoordinator] Exception details: " + Ex.ToString() );
			}

			if( ( Coordinator != null ) &&
				( CoordinatorResponding ) )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Green, " ......... SwarmCoordinator successfully initialized" );
			}
			else
			{
				Log( EVerbosityLevel.Informative, ELogColour.Orange, " ......... SwarmCoordinator failed to be initialized" );
				CoordinatorResponding = false;
			}
		}

		///////////////////////////////////////////////////////////////////////

		/**
		 * Sets the current directory to the same as the one used by the ProcessId
		 */
		public void SetCurrentDirectoryByProcessID (int ProcessID)
		{
			Environment.CurrentDirectory = GetProcessPathById(ProcessID);
		}

		/**
		 * Get a new unique handle from the coordinator, if possible. If not,
		 * generate one locally that we can still use for non-distributed work
		 */
		private Object GetUniqueHandleLock = new Object();
		private Random GetUniqueHandleGenerator = new Random();
		private Int32 GetUniqueHandle()
		{
			StartTiming( "GetUniqueHandle-Internal", true );

			Int32 UniqueHandle = 0;

			// If we have an active connection to the coordinator, ask it for
			// a new handle value
			if( ( Coordinator != null ) &&
				( CoordinatorResponding ) &&
				( AgentApplication.Options.EnableStandaloneMode == false ) )
			{
				// Try to get the value from the coordinator
				try
				{
#if !__MonoCS__ // @todo Mac
					UniqueHandle = Coordinator.GetUniqueHandle();
#endif
				}
				catch( Exception Ex )
				{
					// If there's any error here, fall through to the local handle generation code
					Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[GetUniqueHandle] Communication with the coordinator failed, job distribution will be disabled until the connection is established" );
					Log( EVerbosityLevel.ExtraVerbose, ELogColour.Orange, "[GetUniqueHandle] Exception: " + Ex.ToString() );

					// Until all open connections are closed, the coordinator is abandoned
					CoordinatorResponding = false;
				}
			}

			if( UniqueHandle <= 0 )
			{
				// If the coordinator didn't handle the call above, generate
				// a handle locally that will work for all non-distributed work
				lock( GetUniqueHandleLock )
				{
					do
					{
						// Generate a new random value for the handle
						UniqueHandle = GetUniqueHandleGenerator.Next();
					}
					// Keep generating new values until we find one not already in use
					while( Connections.ContainsKey( UniqueHandle ) );
				}
			}

			StopTiming();

			return UniqueHandle;
		}

		/**
		 * Used by local processes to open a connection to Swarm
		 */
		private Int32 OpenConnection_1_0( Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Always wait until the agent is fully initialized
			Initialized.WaitOne();

			// Unpack the input parameters
			Int32 LocalProcessID = ( Int32 )InParameters["ProcessID"];
			ELogFlags LoggingFlags = ( ELogFlags )InParameters["LoggingFlags"];

			// Check for optional parameters
			if( InParameters.Contains( "ProcessIsOwner" ) )
			{
				bool LocalProcessIsOwner = ( bool )InParameters["ProcessIsOwner"];
				if( LocalProcessIsOwner )
				{
					OwnerProcessID = LocalProcessID;
				}
			}

			// If a restart or an exit has been requested, stop accepting new connections
			if( AgentIsShuttingDown )
			{
				return Constants.INVALID;
			}

			// If this is a "first" new local connection, clear the log window
			LocalConnectionRequestWaiting.Set();
			lock( Connections )
			{
				LocalConnectionRequestWaiting.Reset();

				Int32 LocalConnectionCount = 0;
				foreach( Connection NextConnection in Connections.Values )
				{
					if( NextConnection is LocalConnection )
					{
						LocalConnectionCount++;
					}
				}
				if( LocalConnectionCount == 0 )
				{
					AgentApplication.ClearLogWindow();
				}
			}

			CreateTimings( LoggingFlags );

			// Get a new unique handle that this potential connection will be known by
			Int32 NewConnectionHandle = GetUniqueHandle();

			// Local connection request, the connection ID is the process ID of the caller
			LocalConnection NewLocalConnection = new LocalConnection( NewConnectionHandle, LocalProcessID );

			// Determine if this connection is a replacement of another, existing connection
			foreach( Connection ExistingConnection in Connections.Values )
			{
				if( ExistingConnection is LocalConnection )
				{
					LocalConnection ExistingLocalConnection = ExistingConnection as LocalConnection;
					if( ExistingLocalConnection.ProcessID == LocalProcessID )
					{
						// If this process already has a connection, close the older one
						// since generally the only way this happens is when a connection
						// dies and is re-established later
						Log( EVerbosityLevel.Informative, ELogColour.Orange, "[Connection] Detected new local connection from same process ID as an existing one, closing the old one" );
						Hashtable OldInParameters = null;
						Hashtable OldOutParameters = null;
						CloseConnection( ExistingConnection.Handle, OldInParameters, ref OldOutParameters );
					}
				}
			}

			// Determine if this connection is a child of another, existing connection
			Connection ParentConnection = null;
			AgentJob ParentJob = null;
			foreach( AgentJob Job in ActiveJobs.Values )
			{
				if( Job.CurrentState == AgentJob.JobState.AGENT_JOB_RUNNING )
				{
					// If the incoming connection is from a process spawned by a Job
					// associated with an existing connection, note the parent-child
					// relationship
					lock( Job.ProcessObjectLock )
					{
						if( ( Job.ProcessObject != null ) &&
							( Job.ProcessObject.Id == LocalProcessID ) )
						{
							// Grab the parent and its Job
							ParentConnection = Job.Owner;
							ParentJob = Job;

							// Found what we want, break out
							break;
						}
					}

					// If the Job was marked as manually started, then try to match
					// based on the full path name of the executable
					EJobTaskFlags JobFlags = Job.Specification.JobFlags;
					if( ( JobFlags & EJobTaskFlags.FLAG_MANUAL_START ) != 0 )
					{
						Process ProcessObject = Process.GetProcessById( LocalProcessID );
						string ProcessFilename = Path.GetFileName( ProcessObject.MainModule.FileName );
						string OriginalExecutableName;
						if( Job.Specification.DependenciesOriginalNames.TryGetValue( Job.Specification.ExecutableName, out OriginalExecutableName ) )
						{
                            // Compare in a way that allows matching with the debug executable, which contains the release executable name
                            //@todo - detect debug executable in a robust method, possibly through optional dependencies
                            string OriginalExecutableWithoutExtention = System.IO.Path.GetFileNameWithoutExtension(OriginalExecutableName);
							if (OriginalExecutableWithoutExtention.Contains(ProcessFilename) || ProcessFilename.Contains(OriginalExecutableWithoutExtention))
							{
								Log( EVerbosityLevel.Informative, ELogColour.Green, "[Job] Attaching new process handle to existing Job" );
								ProcessObject.EnableRaisingEvents = true;
								ProcessObject.Exited += new EventHandler( Job.ExitedProcessEventHandler );
								Job.ProcessObject = ProcessObject;

								// Grab the parent and its Job
								ParentConnection = Job.Owner;
								ParentJob = Job;

								// Found what we want, break out
								break;
							}
						}
					}
				}
			}

			// If we found a parent connection, establish the parent-child relationship
			if( ParentConnection != null )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Green, "[Job] Found a parent connection for PID " + LocalProcessID );
				Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[Job]     {0:X8} -> {1:X8}", ParentConnection.Handle, NewConnectionHandle ) );

				NewLocalConnection.Parent = ParentConnection;
				NewLocalConnection.Job = ParentJob;
				ParentConnection.LocalChildren.Add( NewConnectionHandle, NewLocalConnection );
				ParentConnection.LocalChildrenSeen++;
				ParentConnection.ChildrenSeen++;
			}

			// There are times we want to ensure no new connections are coming online
			// where we'll lock the Connections dictionary (see MaintainCache)
			LocalConnectionRequestWaiting.Set();
			lock( Connections )
			{
				LocalConnectionRequestWaiting.Reset();

				// Add the new local connection to the list of tracked connections
				Connections.Add( NewConnectionHandle, NewLocalConnection );
				NewLocalConnection.CurrentState = ConnectionState.CONNECTED;
			}

			// If this is an Instigator's connection, update some state
			if( ParentConnection == null )
			{
				// Update our state and ping the coordinator, if this is an instigating connection
				CurrentState = AgentState.Working;
				WorkingFor = System.Net.Dns.GetHostName();
				PingCoordinator( true );
			}

			// Fill in the connection configuration output parameter
			OutParameters = new Hashtable();
			OutParameters["Version"] = ESwarmVersionValue.VER_1_0;
			OutParameters["AgentProcessID"] = AgentProcessID;
			OutParameters["AgentCachePath"] = GetCacheLocation();
			
			// There are two requirements to be a pure local connection: be local and
			// have no remote parents. The main benefit of a pure local connection is
			// that they get to avoid calling into the Agent for the Channel API.
			bool IsPureLocalConnection = true;
			if( NewLocalConnection.Job != null )
			{
				OutParameters["AgentJobGuid"] = NewLocalConnection.Job.JobGuid;
				if( NewLocalConnection.Job.Owner is RemoteConnection )
				{
					IsPureLocalConnection = false;
				}
			}
			OutParameters["IsPureLocalConnection"] = IsPureLocalConnection;

			// Return the handle for the new local connection
			return NewConnectionHandle;
		}

		/**
		 * Used by an agent to open a remote connection to another agent. Remote
		 * connections are bi-directional and this routine, along with its pair,
		 * ConfirmRemoteConnection, are used to resolve all handshaking. Remote
		 * connections are used by Agents to work on Tasks for Jobs they're
		 * managing on behalf of a LocalConnection.
		 */
		public Int32 OpenRemoteConnection( string RequestingAgentName, Int32 ConnectionHandle, ELogFlags LoggingFlags, bool IsAssigned, DateTime AssignedTimestamp )
		{
			// Always wait until the agent is fully initialized
			Initialized.WaitOne();

			Int32 RemoteConnectionHandle = Constants.INVALID;

			CreateTimings( LoggingFlags );
			StartTiming( "OpenRemoteConn-Internal", true );

			// Make sure we only have one thread in here at a time
			lock( Connections )
			{
				// Reject any incoming remote connection if:
				// we're in standalone mode
				// of if the CPU is already busy
				// or if a restart/exit has been requested
				if( ( AgentApplication.Options.EnableStandaloneMode ) ||
					( CurrentState == AgentState.Busy ) ||
					( AgentIsShuttingDown ) )
				{
					return Constants.INVALID;
				}
				else
				{
					// Determine if this is a request for a connection or an assignment
					if( IsAssigned )
					{
						// Make sure the assignment isn't stale
						if( AssignedTimestamp <= LastAssignedTime )
						{
							return Constants.INVALID;
						}
						if( Connections.Count != 0 )
						{
							// If there are any active jobs running in which the owner is the instigator,
							// meaning that this agent is connected directly to the instigator, reject
							foreach( AgentJob Job in ActiveJobs.Values )
							{
								if( Job.OwnerIsInstigator )
								{
									return Constants.INVALID;
								}
							}

							// Close any existing remote connections in favor of this incoming one
							foreach( Connection ExistingConnection in Connections.Values )
							{
								if( ExistingConnection is RemoteConnection )
								{
									Log( EVerbosityLevel.Informative, ELogColour.Orange, "[Connection] Detected incoming assigned remote connection, closing existing remote connection" );
									Hashtable OldInParameters = null;
									Hashtable OldOutParameters = null;
									CloseConnection( ExistingConnection.Handle, OldInParameters, ref OldOutParameters );
								}
							}
						}
					}
					else
					{
						// For connection requests, any other connection at all will cause a rejection
						if( Connections.Count != 0 )
						{
							return Constants.INVALID;
						}
					}
				}

				// First, ping the remote host to make sure we can talk to it,
				// starting with verifying the Agent with the Coordinator
				try
				{
					if( ( Coordinator != null ) &&
						( CoordinatorResponding ) &&
						( AgentApplication.Options.EnableStandaloneMode == false ) )
					{
						AgentInfo RequestingAgentInfo = null;

						Hashtable RequestedConfiguration = new Hashtable();
						RequestedConfiguration["Version"] = new Version( 0, 0, 0, 0 );
						RequestedConfiguration["RequestingAgentName"] = System.Net.Dns.GetHostName();
						List<AgentInfo> PotentialRemoteAgents = Coordinator.GetAvailableAgents( RequestedConfiguration );
						foreach( AgentInfo NextAgentInfo in PotentialRemoteAgents )
						{
							if( NextAgentInfo.Name == RequestingAgentName )
							{
								RequestingAgentInfo = NextAgentInfo;
								break;
							}
						}

						if( RequestingAgentInfo != null )
						{
							Debug.Assert( RequestingAgentInfo.Configuration.ContainsKey( "IPAddress" ) );
							string RequestingAgentIPAddress = RequestingAgentInfo.Configuration["IPAddress"].ToString();

							ValidateHostName(RequestingAgentIPAddress);

							// Now, ping the Agent
							if( PingRemoteHost( RequestingAgentName, RequestingAgentIPAddress ) )
							{
								// Get the remote agent's interface object and wrap it
								string RemoteAgentURL = String.Format( "tcp://{0}:{1}/SwarmAgent",
									RequestingAgentIPAddress,
									Properties.Settings.Default.AgentRemotingPort.ToString() );

								Agent RequestingAgentInterface = ( Agent )Activator.GetObject( typeof( Agent ), RemoteAgentURL );
								RemoteConnectionInterfaceWrapper WrappedRequestingAgentInterface = new RemoteConnectionInterfaceWrapper( ConnectionHandle, RequestingAgentName, RequestingAgentIPAddress, RequestingAgentInterface );
								Log( EVerbosityLevel.Informative, ELogColour.Green, "[Connect] Remote agent connection object obtained: " + RequestingAgentName );

								// Confirm the connection
								try
								{
									// Send the connection GUID back to the requesting agent as
									// confirmation that we've received the request and plan to
									// accept it after the handshaking is complete
									RemoteConnectionHandle = WrappedRequestingAgentInterface.ConfirmRemoteConnection( ConnectionHandle );
									if( RemoteConnectionHandle >= 0 )
									{
										// Create the new remote connection object
										RemoteConnection NewRemoteConnection = new RemoteConnection( ConnectionHandle,
																									 RequestingAgentInfo,
																									 WrappedRequestingAgentInterface );

										// There are times we want to ensure no new connections are coming online
										// where we'll lock the Connections dictionary (see MaintainCache). Note
										// that we're already in the lock above...
										Connections.Add( ConnectionHandle, NewRemoteConnection );
										NewRemoteConnection.CurrentState = ConnectionState.CONNECTED;

										// Update our state and ping the coordinator
										CurrentState = AgentState.Working;
										WorkingFor = RequestingAgentName;
										if( IsAssigned )
										{
											// Successful assignment
											LastAssignedTime = AssignedTimestamp;
										}
										PingCoordinator( true );

										Log( EVerbosityLevel.Informative, ELogColour.Green, "[Connect] Remote agent connection confirmed: " + RequestingAgentName );
									}
								}
								catch( Exception Ex )
								{
									Log( EVerbosityLevel.Informative, ELogColour.Red, "[Connect] OpenRemoteConnection: " + Ex.ToString() );
									RemoteConnectionHandle = Constants.ERROR_EXCEPTION;
								}
							}
							else
							{
								// Failed to ping, simply return with an appropriate error code
								Log( EVerbosityLevel.Informative, ELogColour.Red, "[Connect] Failed to ping " + RequestingAgentName + " at " + RequestingAgentIPAddress + " to confirm the remote connection" );
								RemoteConnectionHandle = Constants.ERROR_CONNECTION_NOT_FOUND;
							}
						}
						else
						{
							// Failed to find the Agent info for the requesting Agent, simply return with an appropriate error code
							Log( EVerbosityLevel.Informative, ELogColour.Red, "[Connect] Failed to lookup " + RequestingAgentName + " in the Coordinator" );
							RemoteConnectionHandle = Constants.ERROR_CONNECTION_NOT_FOUND;
						}
					}
					else
					{
						// Could not contact the Coordinator, simply return with an appropriate error code
						Log( EVerbosityLevel.Informative, ELogColour.Red, "[Connect] Failed to contact the Coordinator" );
						RemoteConnectionHandle = Constants.ERROR_CONNECTION_NOT_FOUND;
					}
				}
				catch( Exception Ex )
				{
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[Connect] Remote agent connection failed: " + RequestingAgentName );
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[Connect] Exception details: " + Ex.ToString() );
				}

				if( RemoteConnectionHandle < 0 )
				{
					// If we get here, we have failed to create the connection
					Log( EVerbosityLevel.Informative, ELogColour.Red, "[Connect] Remote agent connection failed: " + RequestingAgentName );
				}

				StopTiming();
			}
			return RemoteConnectionHandle;
		}

		/**
		 * Used to confirm a bi-directional remote connection request (from OpenRemoteConnection)
		 */
		public Int32 ConfirmRemoteConnection( Int32 ConnectionHandle )
		{
			StartTiming( "ConfirmRemoteConn-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			// Use the provided connection handle to look up the connection
			// in the set of pending connections
			RemoteConnection PendingConnection;
			if( PendingConnections.TryGetValue( ConnectionHandle, out PendingConnection ) )
			{
				// Found the connection, remove it from the set of pending connections
				PendingConnections.Remove( ConnectionHandle );

				// There are times we want to ensure no new connections are coming online
				// where we'll lock the Connections dictionary (see MaintainCache)
				lock( Connections )
				{
					// Move it to the set of active connections and mark it fully connected
					Connections.Add( ConnectionHandle, PendingConnection );
					PendingConnection.CurrentState = ConnectionState.CONNECTED;
				}

				ErrorCode = Constants.SUCCESS;
			}

			StopTiming();
			// If we get here, we have failed to confirm the connection
			return ErrorCode;
		}

		public bool AgentNamePassesAllowedAgentsFilter( string RemoteAgentName )
		{
			bool RemoteAgentAllowed = false;

			// First, see if the name matches any of the user-specified allowed machine names
			string[] AdditionalRemoteAgents = AgentApplication.Options.AllowedRemoteAgentNames.Split( new char[] { ' ', ',', ';' } );
			foreach( string AllowedRemoteAgentName in AdditionalRemoteAgents )
			{
				string AllowedRemoteAgentNameCaps = AllowedRemoteAgentName.ToUpperInvariant();
				string RemoteAgentNameCaps = RemoteAgentName.ToUpperInvariant();

				// If it ends in a wildcard, match the beginning of the string
				if( AllowedRemoteAgentNameCaps.EndsWith( "*" ) )
				{
					string TrimmedPrefixName = AllowedRemoteAgentNameCaps.Substring( 0, AllowedRemoteAgentNameCaps.Length - 1 );
					if( RemoteAgentNameCaps.StartsWith( TrimmedPrefixName ) )
					{
						RemoteAgentAllowed = true;
						break;
					}
				}
				else if( RemoteAgentNameCaps == AllowedRemoteAgentNameCaps )
				{
					RemoteAgentAllowed = true;
					break;
				}
			}

			return RemoteAgentAllowed;
		}

		/**
		 * Utility for simply pinging a remote host to see if it's alive.
		 * Only returns true if the host is pingable, false for all
		 * other cases.
		 */
		private bool PingRemoteHost( string RemoteHostName, string RemoteHostNameOrAddressToPing )
		{
			bool PingSuccess = false;
			try
			{
				// The default Send method will timeout after 5 seconds
				Ping PingSender = new Ping();
				PingReply Reply = PingSender.Send( RemoteHostNameOrAddressToPing );
				if( Reply.Status == IPStatus.Success )
				{
					AgentApplication.Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, "[PingRemoteHost] Successfully pinged " + RemoteHostName + " with " + RemoteHostNameOrAddressToPing );
					PingSuccess = true;
				}
			}
			catch( Exception Ex )
			{
				// This will catch any error conditions like unknown hosts
				AgentApplication.Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, "[PingRemoteHost] Failed to ping " + RemoteHostName + " with " + RemoteHostNameOrAddressToPing );
				AgentApplication.Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, "[PingRemoteHost] Exception details: " + Ex.Message );
			}
			return PingSuccess;
		}

		/**
		 * Used internally to try to open connections to remote agents
		 */
		public Int32 TryOpenRemoteConnection( LocalConnection ParentConnection, AgentInfo RemoteAgentInfo, bool IsAssigned, DateTime AssignedTimestamp, out RemoteConnection OpenedRemoteConnection )
		{
			// Initialize the return values
			Int32 ErrorCode = Constants.INVALID;
			OpenedRemoteConnection = null;

			string RemoteAgentName = RemoteAgentInfo.Name;
			string RemoteAgentIPAddress = RemoteAgentInfo.Configuration["IPAddress"].ToString();

			Log( EVerbosityLevel.Verbose, ELogColour.Green, "[Connect] Trying to open a remote connection to " + RemoteAgentName + " at " + RemoteAgentIPAddress );

			bool bHostNameValid = true;
			try
			{
				ValidateHostName(RemoteAgentIPAddress);
			}
			catch (Exception)
			{
				ErrorCode = Constants.ERROR_EXCEPTION;
				bHostNameValid = false;
			}

			// First, ping the remote host to make sure we can talk to it
			if (bHostNameValid && PingRemoteHost(RemoteAgentName, RemoteAgentIPAddress))
			{
				// Get the new unique handle that this connection will be known by
				Int32 NewConnectionHandle = GetUniqueHandle();

				// Get the remote Agent interface object
				string RemoteAgentURL = String.Format( "tcp://{0}:{1}/SwarmAgent",
					RemoteAgentIPAddress,
					Properties.Settings.Default.AgentRemotingPort.ToString() );

				Agent RemoteAgentInterface = ( Agent )Activator.GetObject( typeof( Agent ), RemoteAgentURL );
				RemoteConnectionInterfaceWrapper WrappedRemoteAgentInterface = new RemoteConnectionInterfaceWrapper( NewConnectionHandle, RemoteAgentName, RemoteAgentIPAddress, RemoteAgentInterface );
				Log( EVerbosityLevel.Verbose, ELogColour.Green, "[Connect] Remote agent interface object obtained for " + RemoteAgentName + " at " + RemoteAgentIPAddress );

				// With the interface in hand, try to open a bi-directional connection.

				// Create the new RemoteConnection object and add it to the set
				// of pending connections, waiting for the incoming confirmation
				// which will move it to the active connection set. See
				// ConfirmRemoteConnection for where this happens.
				RemoteConnection NewRemoteConnection = new RemoteConnection( NewConnectionHandle,
																			 RemoteAgentInfo,
																			 WrappedRemoteAgentInterface );
				PendingConnections.Add( NewConnectionHandle, NewRemoteConnection );

				// Try to open the connection
				try
				{
					// Send the connection handle and the machine name that will
					// be used to construct the return URL for bi-directional
					// connection
					if( WrappedRemoteAgentInterface.OpenRemoteConnection(System.Net.Dns.GetHostName(), NewConnectionHandle, LoggingFlags, IsAssigned, AssignedTimestamp ) >= 0 )
					{
						// Successful confirmation, double check that the connection has moved
						if( PendingConnections.ContainsKey( NewConnectionHandle ) == false )
						{
							if( Connections.ContainsKey( NewConnectionHandle ) )
							{
								// Connection was successfully moved to the active connection set
								// thus the bi-directional communication link is established
								Log( EVerbosityLevel.Informative, ELogColour.Green, "[Connect] Successfully opened a remote connection with " + RemoteAgentName );

								// Establish the parent-child relationship
								NewRemoteConnection.Parent = ParentConnection;
								NewRemoteConnection.Job = ParentConnection.Job;
								ParentConnection.RemoteChildren.Add( NewConnectionHandle, NewRemoteConnection );
								ParentConnection.RemoteChildrenSeen++;
								ParentConnection.ChildrenSeen++;

								// Update the visualizer indicating that another remote connection is online
								if( NewRemoteConnection.Job.OwnerIsInstigator )
								{
									AgentApplication.UpdateMachineState( RemoteAgentName, -1, EProgressionState.RemoteConnected );
								}

								// Return the new handle for this connection
								OpenedRemoteConnection = NewRemoteConnection;
								ErrorCode = Constants.SUCCESS;
							}
							else
							{
								// Woah. Bad news.
							}
						}
					}
					else
					{
						// The connection failed to open, but didn't timeout and die, so it's likely just busy.
						// Return with a success code, but with no remote agent, signaling that it's possible
						// to try this one again later.
						ErrorCode = Constants.SUCCESS;
					}
				}
				catch( Exception Ex )
				{
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[Connect] Exception: " + RemoteAgentName + ": " + Ex.ToString() );
					ErrorCode = Constants.ERROR_EXCEPTION;
				}

				if( ErrorCode < 0 )
				{
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[Connect] Failed to open a remote connection with " + RemoteAgentName );
				}

				// Regardless of how we got here, make sure the connection has been removed
				// from the pending set (this is a safe call to make anytime)
				PendingConnections.Remove( NewConnectionHandle );
			}
			else
			{
				// Failed to ping, simply return with an appropriate error code
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			return ErrorCode;
		}

		/**
		 * Used by both local and remote clients to close their connection to Swarm
		 */
		private Object CloseConnectionLock = new Object();
		public Int32 CloseConnection_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			Int32 ErrorCode = Constants.INVALID;

			Connection ConnectionToClose;
			if( Connections.TryGetValue( ConnectionHandle, out ConnectionToClose ) )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Closing connection {0:X8} using handle {1:X8}", ConnectionToClose.Handle, ConnectionHandle ) );

				// Only try to close connected connections. Most common case of getting
				// an inactive connection here is closing a bi-directional remote
				// connection, which closes you back when you close it.
				bool AllowDisconnection = false;
				lock( CloseConnectionLock )
				{
					if( ConnectionToClose.CurrentState == ConnectionState.CONNECTED )
					{
						// Update the state and continue
						Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Connection confirmed for disconnection {0:X8}", ConnectionToClose.Handle ) );
						ConnectionToClose.CurrentState = ConnectionState.DISCONNECTING;
						AllowDisconnection = true;
					}
					else
					{
						Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Connection already being disconnected, nothing to do {0:X8}", ConnectionToClose.Handle ) );
					}
				}

				// If we're allowed to disconnect, continue on in
				if( AllowDisconnection )
				{
					// Before we evaluate any state as part of closing this connection,
					// make sure all incoming messages in the queue have been processed
					FlushMessageQueue( ConnectionToClose, false );

					// If this connection has a job, handle fallout from closing this connection
					if( ConnectionToClose.Job != null )
					{
						// If this connection is the owner of a job, close it
						if( ConnectionToClose.Job.Owner == ConnectionToClose )
						{
							// If this is the instigator, then this job should have been closed already
							if( ConnectionToClose.Job.OwnerIsInstigator )
							{
								Log( EVerbosityLevel.Informative, ELogColour.Orange, String.Format( "[CloseConnection] Closing orphaned Job ({0})", ConnectionToClose.Job.JobGuid ) );
							}
							ConnectionToClose.Job.CloseJob();
						}
						else
						{
							// Otherwise, be sure to cancel all reservations for this connection
							ConnectionToClose.Job.CancelReservations( ConnectionToClose );

							// And make all active tasks assigned out to this connection fail
							foreach( AgentTask NextTask in ConnectionToClose.RunningTasks.Values )
							{
								ConnectionToClose.Job.UpdateTaskState( new AgentTaskState( NextTask.Specification.JobGuid,
																						   NextTask.Specification.TaskGuid,
																						   EJobTaskState.TASK_STATE_KILLED ) );
							}

							// If this is a remote worker connection closing, note it for the visualizer
							if( ( ConnectionToClose.Job.OwnerIsInstigator ) &&
								( ConnectionToClose is RemoteConnection ) )
							{
								// A remote worker agent connection is closing
								RemoteConnection RemoteConnectionToClose = ConnectionToClose as RemoteConnection;
								lock( AbandonPotentialRemoteAgentsLock )
								{
									if( ( RemoteConnectionToClose.Parent != null ) &&
										( RemoteConnectionToClose.Parent is LocalConnection ) &&
										( ( RemoteConnectionToClose.Parent as LocalConnection ).RemoteAgentsAbandoned == false ) )
									{
										// If possible, add the closed connection back to the list of potentially available agents
										RemoteConnectionToClose.Info.Configuration["AssignedTo"] = "";

										LocalConnection ConnectionToCloseParent = RemoteConnectionToClose.Parent as LocalConnection;
										ConnectionToCloseParent.UnavailableRemoteAgents.Enqueue( RemoteConnectionToClose.Info );
	
										AgentApplication.UpdateMachineState( RemoteConnectionToClose.Info.Name, -1, EProgressionState.Blocked );
									}
									else
									{
										AgentApplication.UpdateMachineState( RemoteConnectionToClose.Info.Name, -1, EProgressionState.RemoteDisconnected );
									}
								}
							}
						}

						// If this is the local job application, update the stats one last time
						if( ( ConnectionToClose.Job.ProcessObject != null ) &&
							( ConnectionToClose is LocalConnection ) &&
							( ConnectionToClose.Parent != null ) )
						{
							try
							{
								LocalConnection LocalConnectionToClose = ConnectionToClose as LocalConnection;
								if( LocalConnectionToClose.ProcessID == ConnectionToClose.Job.ProcessObject.Id )
								{
									LocalConnectionToClose.Job.ProcessObjectPeakVirtualMemorySize64 = LocalConnectionToClose.Job.ProcessObject.PeakVirtualMemorySize64;
									LocalConnectionToClose.Job.ProcessObjectPeakWorkingSet64 = LocalConnectionToClose.Job.ProcessObject.PeakWorkingSet64;
								}
							}
							catch( Exception )
							{
								// The process could exit while we're updating these values, it's ok
							}
						}

						ConnectionToClose.Job = null;

						// Flush the queue to make sure any messages generated by closing the job have been delivered
						FlushMessageQueue( ConnectionToClose, false );
					}

					// Orphan any local children, assuming they'll either clean themselves
					// up or be forcibly closed at a later time
					foreach( LocalConnection LocalChild in ConnectionToClose.LocalChildren.Values )
					{
						Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Orphaning local connection ({0:X8} is the parent of {1:X8})", ConnectionToClose.Handle, LocalChild.Handle ) );
						LocalChild.Parent = null;
						ConnectionToClose.LocalChildren.Remove( LocalChild.Handle );
					}
					Debug.Assert( ConnectionToClose.LocalChildren.Count == 0 );

					// Remote children of this connection are always closed here since
					// they cannot close themselves
					foreach( RemoteConnection RemoteChild in ConnectionToClose.RemoteChildren.Values )
					{
						// For each child connection, orphan the child then simply recur to close it
						Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Recursively closing remote connection ({0:X8} is the parent of {1:X8})", ConnectionToClose.Handle, RemoteChild.Handle ) );
						RemoteChild.Parent = null;

						Hashtable ChildInParameters = null;
						Hashtable ChildOutParameters = null;
						CloseConnection( RemoteChild.Handle, ChildInParameters, ref ChildOutParameters );
					}
					// Note: we cannot assert that all remote children are gone here, since aliases
					// may remain even after closing the original connection. Aliases are cleaned
					// up in MaintainConnections

					// Local and remote connections have slightly different additional work to do.
					// Everything that requires we still be "active" should be done at this time.
					// Everything that requires we be marked "inactive" should be done later.
					if( ConnectionToClose is LocalConnection )
					{
						// Do any shutdown and cleanup work specific to local connections
						LocalConnection LocalConnectionToClose = ConnectionToClose as LocalConnection;

						// If we're closing a child connection, remove from the parent
						Connection LocalConnectionToCloseParent = LocalConnectionToClose.Parent;
						if( LocalConnectionToCloseParent != null )
						{
							// Remove this connection from the parent's child set
							LocalConnectionToCloseParent.LocalChildren.Remove( LocalConnectionToClose.Handle );
						}
						else
						{
							// Otherwise, send back any timing data to this instigator
							DumpTimings( LocalConnectionToClose );
						}
					}
					else
					{
						Debug.Assert( ConnectionToClose is RemoteConnection );
						RemoteConnection RemoteConnectionToClose = ConnectionToClose as RemoteConnection;

						// If this is a child conection, remove it from the parent's child set
						Connection RemoteConnectionToCloseParent = RemoteConnectionToClose.Parent;
						if( RemoteConnectionToCloseParent != null )
						{
							RemoteConnectionToCloseParent.RemoteChildren.Remove( RemoteConnectionToClose.Handle );
						}

						// For remote connections, close the connection on the other end and let
						// that action propagate through all state there
						Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Closing bi-directional remote connection ({0:X8})", RemoteConnectionToClose.Handle ) );
						Hashtable RemoteInParameters = null;
						Hashtable RemoteOutParameters = null;
						RemoteConnectionToClose.Interface.CloseConnection( RemoteConnectionToClose.Handle, RemoteInParameters, ref RemoteOutParameters );
						Log( EVerbosityLevel.Verbose, ELogColour.Green, String.Format( "[CloseConnection] Bi-directional remote connection closed ({0:X8})", RemoteConnectionToClose.Handle ) );
					}

					// Attempt to flush the message queue and mark the connection as disconnected
					if( FlushMessageQueue( ConnectionToClose, true ) == false )
					{
						// If that fails, just mark it disconnected anyway (not as safe)
						Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Fallback path: Connection disconnected {0:X8}", ConnectionToClose.Handle ) );
						ConnectionToClose.CurrentState = ConnectionState.DISCONNECTED;
                        ConnectionToClose.DisconnectedTime = DateTime.UtcNow;
					}

					// Now, do anything that needs to/can be done after the connection is marked inactive
					if( ConnectionToClose is LocalConnection )
					{
						// For local connections, give the message queue a final nudge to make
						// sure it exits properly - it relies on getting one more release than
						// it has messages to ensure it'll get a "no more messages" event
						LocalConnection LocalConnectionToClose = ConnectionToClose as LocalConnection;
						LocalConnectionToClose.MessageAvailableSignal();
					}
				}
				ErrorCode = Constants.SUCCESS;
			}
			else
			{
				Log( EVerbosityLevel.Critical, ELogColour.Red, String.Format( "[CloseConnection] Rejected, unrecognized or inactive connection ({0:X8})", ConnectionHandle ) );
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			return ( ErrorCode );
		}

		/**
		 * Requeues the previously unavailable agent list
		 */
		public bool RetryUnavailableRemoteAgent( LocalConnection ParentConnection )
		{
			if( ParentConnection.UnavailableRemoteAgents.Count > 0 )
			{
				// Queue up the next unavailable agent
				ParentConnection.PotentialRemoteAgents.Enqueue( ParentConnection.UnavailableRemoteAgents.Dequeue() );
				return true;
			}
			return false;
		}

		/**
		 * Refresh the list of active remote agents
		 */
		public bool ResetPotentialRemoteAgents( LocalConnection ParentConnection )
		{
			StartTiming( "GetRemoteAgents-Internal", true );

			// Start everything off empty
			List<AgentInfo> UnattachedRemoteAgents = new List<AgentInfo>();

			if( ( Coordinator != null ) &&
				( CoordinatorResponding ) &&
				( AgentApplication.Options.EnableStandaloneMode == false ) )
			{
				List<AgentInfo> PotentialRemoteAgents = null;
				try
				{
					// Get all alive agents - but don't connect to them just yet
					Hashtable RequestedConfiguration = new Hashtable();
					RequestedConfiguration["Version"] = CurrentVersion;
					RequestedConfiguration["GroupName"] = AgentApplication.Options.AllowedRemoteAgentGroup;
					RequestedConfiguration["RequestingAgentName"] = System.Net.Dns.GetHostName();
					RequestedConfiguration["RequestAssignmentFor"] = System.Net.Dns.GetHostName();
					PotentialRemoteAgents = Coordinator.GetAvailableAgents( RequestedConfiguration );
				}
				catch( Exception )
				{
					// Until all open connections are closed, the coordinator is abandoned
					CoordinatorResponding = false;
				}

				// If we survived the call to the coordinator, work the list
				if( ( CoordinatorResponding ) &&
					( PotentialRemoteAgents != null ) )
				{
					// Filter the list by all currently attached remote agents and ourself
					foreach( AgentInfo Info in PotentialRemoteAgents )
					{
						bool WorkerAlreadyConnectedOrDisallowed = false;
						if( ( Info.Name == System.Net.Dns.GetHostName()) ||
							( !AgentNamePassesAllowedAgentsFilter( Info.Name ) ) )
						{
							// Trivial case
							WorkerAlreadyConnectedOrDisallowed = true;
						}
						else
						{
							// Walk through the list of already attached workers
							foreach( RemoteConnection Remote in ParentConnection.RemoteChildren.Values )
							{
								if( Remote.Info.Name == Info.Name )
								{
									WorkerAlreadyConnectedOrDisallowed = true;
									break;
								}
							}
						}
						// If not already attached, add it to the new list of potentials
						if( !WorkerAlreadyConnectedOrDisallowed )
						{
							UnattachedRemoteAgents.Add( Info );
						}
					}
				}
			}

			// Set the newly crafted list of potential workers
			ParentConnection.PotentialRemoteAgents = new Queue<AgentInfo>( UnattachedRemoteAgents );
			ParentConnection.UnavailableRemoteAgents.Clear();

			// Log out which other machines we might be able to ask for help
			foreach( AgentInfo NextAgent in UnattachedRemoteAgents )
			{
				Log( EVerbosityLevel.Complex, ELogColour.Blue, String.Format( "[Connect] Coordinator named remote agent {0} which is {1}", NextAgent.Name, NextAgent.State ) );
			}

			StopTiming();

			return ( UnattachedRemoteAgents.Count > 0 );
		}

		/**
		 * Pulls the next agent off the list of potentials and tries to open a connection
		 * until it either opens one successfully or runs out of agents
		 */
		public RemoteConnection GetNextRemoteAgent( LocalConnection ParentConnection )
		{
			// Walk through the list of potential agents and try to open connections
			// until we either find one or run out of agents
			RemoteConnection Remote = null;
			AgentJob Job = ParentConnection.Job;
			while( ( Job != null ) &&
				   ( Job.CurrentState == AgentJob.JobState.AGENT_JOB_RUNNING ) &&
				   ( ParentConnection.PotentialRemoteAgents.Count > 0 ) &&
				   ( Remote == null ) )
			{
				AgentInfo NextRemoteAgent = ParentConnection.PotentialRemoteAgents.Dequeue();

				bool IsAssigned = false;
				DateTime AssignedTimestamp = DateTime.MinValue;
				if( ( NextRemoteAgent.Configuration.ContainsKey( "AssignedTo" ) ) &&
					( ( NextRemoteAgent.Configuration["AssignedTo"] as string ) == System.Net.Dns.GetHostName()) )
				{
					IsAssigned = true;
					AssignedTimestamp = ( DateTime )NextRemoteAgent.Configuration["AssignedTime"];
				}

				// If we failed to open the connection, but the agent is alive, add it to the list to try again later
				if( ( TryOpenRemoteConnection( ParentConnection, NextRemoteAgent, IsAssigned, AssignedTimestamp, out Remote ) == Constants.SUCCESS ) &&
					( Remote == null ) )
				{
					AgentApplication.UpdateMachineState( NextRemoteAgent.Name, -1, EProgressionState.Blocked );
					ParentConnection.UnavailableRemoteAgents.Enqueue( NextRemoteAgent );
				}
			}

			// Return whatever we found
			return Remote;
		}

		public Object AbandonPotentialRemoteAgentsLock = new Object();
		public void AbandonPotentialRemoteAgents( LocalConnection ParentConnection )
		{
			lock( AbandonPotentialRemoteAgentsLock )
			{
				// Walk through the list of potential agents and disconnect each one
				foreach( AgentInfo NextRemoteAgent in ParentConnection.PotentialRemoteAgents )
				{
					AgentApplication.UpdateMachineState( NextRemoteAgent.Name, -1, EProgressionState.RemoteDisconnected );
				}
				// Done with the queue now, clear it
				ParentConnection.PotentialRemoteAgents.Clear();

				foreach( AgentInfo NextRemoteAgent in ParentConnection.UnavailableRemoteAgents )
				{
					AgentApplication.UpdateMachineState( NextRemoteAgent.Name, -1, EProgressionState.RemoteDisconnected );
				}
				// Done with the queue now, clear it
				ParentConnection.UnavailableRemoteAgents.Clear();

				// Finished with remote agents, set the flag
				ParentConnection.RemoteAgentsAbandoned = true;
			}
		}

		private void PrintSimpleAgentList( List<AgentInfo> AgentInfoList )
		{
			List<string> AgentStateMessages = new List<string>();

			// For each agent in the list, print out the name and state
			foreach( AgentInfo NextAgent in AgentInfoList )
			{
				string StateMessage = "";
				if( NextAgent.State == AgentState.Working )
				{
					if( ( NextAgent.Configuration.ContainsKey( "WorkingFor" ) ) &&
						( ( NextAgent.Configuration["WorkingFor"] as string ) != "" ) )
					{
						StateMessage = "Working for " + NextAgent.Configuration["WorkingFor"];
					}
					else
					{
						StateMessage = "Working for an unknown Agent";
					}
				}
				else
				{
					StateMessage = NextAgent.State.ToString();
				}
				if( ( NextAgent.Configuration.ContainsKey( "AssignedTo" ) ) &&
					( ( NextAgent.Configuration["AssignedTo"] as string ) != "" ) )
				{
					StateMessage += ", Assigned to " + NextAgent.Configuration["AssignedTo"];
				}
				else
				{
					StateMessage += ", Unassigned";
				}

				// The coordinator always tracks the IP address of each Agent - we rely on this
				Debug.Assert( NextAgent.Configuration.ContainsKey( "IPAddress" ) );
				string AgentIPAddressCoordinator = NextAgent.Configuration["IPAddress"].ToString();
				Debug.Assert( NextAgent.Configuration.ContainsKey( "UserName" ) );
				string AgentUserName = NextAgent.Configuration["UserName"].ToString();

				AgentStateMessages.Add( "    " + NextAgent.Name + " (User = " + AgentUserName + ", IP = " + AgentIPAddressCoordinator + ", Version = " + NextAgent.Version.ToString() + ") is currently " + StateMessage );
				
				// Verify the IP address we got from the coordinator and report any errors
				bool AgentIPAddressVerified = false;
				try
				{
					IPHostEntry AgentHostEntry = Dns.GetHostEntry( NextAgent.Name );
					if( AgentHostEntry.AddressList.Length > 0 )
					{
						if( AgentHostEntry.AddressList[AgentHostEntry.AddressList.Length - 1].ToString() == AgentIPAddressCoordinator )
						{
							AgentIPAddressVerified = true;
						}
					}
				}
				catch( Exception )
				{
					// Whatever the exception is, we have failed to verify the IP address
				}
				if( !AgentIPAddressVerified )
				{
					AgentStateMessages.Add( "    " + NextAgent.Name + ": IP address for this Agent cannot be verified - Coordinator and local mismatch or DNS inactive!" );
				}
			}

			// Sorts the list by agent name
			AgentStateMessages.Sort();

			foreach( string NextMessage in AgentStateMessages )
			{
				Log( EVerbosityLevel.Informative, ELogColour.Green, NextMessage );
			}
		}

		/**
		 * Gets the list of all remote agents from the Coordinator and pings each of them
		 */
		public void PingRemoteAgents( string AgentGroupName )
		{
			List<AgentInfo> PotentialRemoteAgents = null;
			try
			{
				if( ( Coordinator != null ) &&
					( CoordinatorResponding ) &&
					( AgentApplication.Options.EnableStandaloneMode == false ) )
				{
					Hashtable RequestedConfiguration = new Hashtable();
					RequestedConfiguration["Version"] = CurrentVersion;
					RequestedConfiguration["GroupName"] = AgentGroupName;
					RequestedConfiguration["RequestingAgentName"] = System.Net.Dns.GetHostName();
					PotentialRemoteAgents = Coordinator.GetAvailableAgents( RequestedConfiguration );
					PrintSimpleAgentList( PotentialRemoteAgents );
				}
			}
			catch( Exception )
			{
			}
		}

		public void RestartAgentGroup( string AgentGroupName )
		{
			ThreadPool.QueueUserWorkItem( new WaitCallback( RestartAgentGroupThreadProc ), AgentGroupName );
		}

		public void RestartAgentGroupThreadProc( Object MethodData )
		{
			// Extract the method parameters
			string AgentGroupName = MethodData as string;
			try
			{
				if( ( Coordinator != null ) &&
					( CoordinatorResponding ) &&
					( AgentApplication.Options.EnableStandaloneMode == false ) )
				{
					Hashtable RequestedConfiguration = new Hashtable();
					RequestedConfiguration["Version"] = new Version( 0, 0, 0, 0 );
					RequestedConfiguration["GroupName"] = AgentGroupName;
					RequestedConfiguration["RequestingAgentName"] = System.Net.Dns.GetHostName();

					// Start by getting the current state of the agents in the group we care about
					List<AgentInfo> RemoteAgentsInGroupBefore = Coordinator.GetAvailableAgents( RequestedConfiguration );
					Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Restarting the following agents (in group " + AgentGroupName + ")" );
					PrintSimpleAgentList( RemoteAgentsInGroupBefore );

					// Tell the coordinator to restart them
					Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Restarting now" );
					Coordinator.RestartAgentGroup( AgentGroupName );

					Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Waiting as agents come back online..." );

					// Wait for all of the original agents to restart
					while( RemoteAgentsInGroupBefore.Count > 0 )
					{
						// Print out any newly online agents
						List<AgentInfo> RemoteAgentsInGroupAfter = Coordinator.GetAvailableAgents( RequestedConfiguration );

						List<AgentInfo> RemoteAgentsBackOnline = new List<AgentInfo>();
						foreach( AgentInfo NextAfterAgent in RemoteAgentsInGroupAfter )
						{
							foreach( AgentInfo NextBeforeAgent in RemoteAgentsInGroupBefore )
							{
								if( NextAfterAgent.Name == NextBeforeAgent.Name )
								{
									RemoteAgentsBackOnline.Add( NextBeforeAgent );
								}
							}
						}

						if( RemoteAgentsBackOnline.Count > 0 )
						{
							PrintSimpleAgentList( RemoteAgentsBackOnline );
							foreach( AgentInfo NextBeforeAgent in RemoteAgentsBackOnline )
							{
								RemoteAgentsInGroupBefore.Remove( NextBeforeAgent );
							}
						}

						// Sleep a little bit before checking again
						Thread.Sleep( 1000 );
					}

					AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Requested agent restart complete" );
				}
			}
			catch( Exception )
			{
			}
		}

		/**
		 * Maintain a connection to the coordinator
		 */
		public bool PingCoordinator( bool ForcePing )
		{
			StartTiming( "PingCoordinator-Internal", true );

			bool CoordinatorPinged = false;

			// Determine if we should ping
#if !__MonoCS__ // @todo Mac
			bool ShouldPing =
				( ForcePing ) ||
                (DateTime.UtcNow > LastCoordinatorPingTime + TimeSpan.FromMinutes(1));
#else
			bool ShouldPing = false;
#endif

			if( ShouldPing )
			{
				// Determine if we can ping
				bool CouldPing =
					( Coordinator != null ) &&
					( CoordinatorResponding ) &&
					( PingRemoteHost( AgentApplication.Options.CoordinatorRemotingHost, AgentApplication.Options.CoordinatorRemotingHost ) );

				if( CouldPing )
				{
                    LastCoordinatorPingTime = DateTime.UtcNow;

					// Try to ping the coordinator
					try
					{
						// The AgentInfo defaults cover almost everything, fill in the state
						AgentInfo AgentInfoUpdate = new AgentInfo();
						AgentInfoUpdate.Version = CurrentVersion;

						// We'll only advertise the number of cores we'll have available for remote connections
						AgentInfoUpdate.Configuration["LocalAvailableCores"] = AgentApplication.DeveloperOptions.LocalJobsDefaultProcessorCount;
						AgentInfoUpdate.Configuration["RemoteAvailableCores"] = AgentApplication.DeveloperOptions.RemoteJobsDefaultProcessorCount;

						// Update with our build group, etc.
						AgentInfoUpdate.Configuration["UserName"] = Environment.UserName.ToUpperInvariant();
						AgentInfoUpdate.Configuration["GroupName"] = AgentApplication.Options.AgentGroupName;
						AgentInfoUpdate.Configuration["WorkingFor"] = WorkingFor;

#if !__MonoCS__ // @todo Mac
						IPAddress[] CoordinatorAddresses = Dns.GetHostAddresses(AgentApplication.Options.CoordinatorRemotingHost);
						IPAddress[] LocalAddresses = Dns.GetHostAddresses(Dns.GetHostName());

						if(CoordinatorAddresses.Any(CoordinatorAddress => IPAddress.IsLoopback(CoordinatorAddress) || LocalAddresses.Contains(CoordinatorAddress)))
						{
							AgentInfoUpdate.Configuration["IPAddress"] = IPAddress.Loopback;
						}
						else
						{
							var NetworkInterface = NetworkUtils.GetBestInterface(
								CoordinatorAddresses.Where(CoordinatorAddress => CoordinatorAddress.AddressFamily == AddressFamily.InterNetwork).First()
							);

							AgentInfoUpdate.Configuration["IPAddress"] = NetworkUtils.GetInterfaceIPv4Address(NetworkInterface);
						}
#else
						AgentInfoUpdate.Configuration["IPAddress"] = "192.168.0.203";//.2.9";
#endif

						// Check if standalone mode is enabled and use that unless we're closing down
						if( ( CurrentState != AgentState.Closed ) &&
							( AgentApplication.Options.EnableStandaloneMode ) )
						{
							AgentInfoUpdate.State = AgentState.Standalone;
						}
						else
						{
							// Otherwise, pass through the real state
							AgentInfoUpdate.State = CurrentState;
						}

						// A positive return value from the Ping is a request for restart
						PingResponse Response = Coordinator.Ping( AgentInfoUpdate );
						if( Response == PingResponse.Restart )
						{
							Log( EVerbosityLevel.Informative, ELogColour.Green, "[PingCoordinator] Restart has been requested" );

#if !__MonoCS__
							// Only restart if we're running the published (and restartable) version
							if( ApplicationDeployment.IsNetworkDeployed )
							{
								// Only update the value if we're not already restarting
								if( !AgentIsRestarting )
								{
									// If we've got a signal to restart, request that the agent shutdown
									RequestRestart();
								}
							}
							else
							{
								Log( EVerbosityLevel.Informative, ELogColour.Green, "[PingCoordinator] Restart request ignored for non-published agents" );
							}
#endif
						}
						CoordinatorPinged = true;
					}
					catch( Exception Ex )
					{
						// If we fail for any reason, just log it and try again later
						Log( EVerbosityLevel.Informative, ELogColour.Orange, "[Ping] Communication with the coordinator failed, job distribution will be disabled until the connection is established" );
						Log( EVerbosityLevel.ExtraVerbose, ELogColour.Orange, "Exception details: " + Ex.ToString() );

						// Until all open connections are closed, the coordinator is abandoned
						CoordinatorResponding = false;
					}
				}
				else
				{
					AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[PingCoordinator] Determined that we couldn't ping the coordinator" );
					if( Coordinator == null )
					{
						AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[PingCoordinator] Coordinator is null" );
					}
					else if( CoordinatorResponding == false )
					{
						AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[PingCoordinator] CoordinatorResponding is false" );
					}
					else
					{
						AgentApplication.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[PingCoordinator] Coordinator ping failed" );
					}
				}
			}
			else
			{
				AgentApplication.Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, "[PingCoordinator] Determined that we shouldn't ping the coordinator right now" );
			}

			StopTiming();
			return CoordinatorPinged;
		}

		public void RequestShutdown()
		{
			// Simply set the flag we'll use to signal that we're trying to shutdown
			Log( EVerbosityLevel.Informative, ELogColour.Green, "[RequestShutdown] Shutdown has been requested" );
			AgentIsShuttingDown = true;
		}

		public void RequestRestart()
		{
			// Set the restart flag and request the shutdown that will allow the actual restart
			AgentIsRestarting = true;
			RequestShutdown();
		}

		public bool ShuttingDown()
		{
			// If we're restarting and there are open connections, delay the true
			// result. Otherwise, honesty is the best policy.
			if( ( Restarting() == false ) ||
				( Connections.Count == 0 ) )
			{
				// This value is set in either PingCoordinator or from the
				// main application as a result of user interaction
				return AgentIsShuttingDown;
			}
			return false;
		}

		public bool Restarting()
		{
			// This value is set in PingCoordinator
			return AgentIsRestarting;
		}

		public void MaintainConnections()
		{
			// If we're not initialied yet, do nothing
			if( !Initialized.WaitOne( 0 ) )
			{
				return;
			}

			// Ping the coordinator periodically
			PingCoordinator( false );

			// For each active connection
			List<Connection> DroppedConnections = new List<Connection>();
			List<Connection> ClosedAndInactiveConnections = new List<Connection>();

			foreach( KeyValuePair<Int32, Connection> NextHandleAndConnection in Connections.Copy() )
			{
				Int32 NextHandle = NextHandleAndConnection.Key;
				Connection NextConnection = NextHandleAndConnection.Value;

				Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, String.Format( "[MaintainConnections] Maintaining Connection ({0:X8})", NextHandle ) );

				// For connections that are still connected, we mostly just look
				// for dropped connections to clean up if necessary
				if( NextConnection.CurrentState == ConnectionState.CONNECTED )
				{
					if( NextConnection is LocalConnection )
					{
						LocalConnection Local = NextConnection as LocalConnection;

						// For each local connection, verify the process is still running
						bool bProcessIsStillRunning = false;
						Process ProcessObject = null;
						try
						{
							ProcessObject = Process.GetProcessById( Local.ProcessID );
							if( ProcessObject.HasExited == false )
							{
								// Connection is still alive
								bProcessIsStillRunning = true;
							}
						}
						catch( Exception )
						{
							// Process no longer exists, fall through
						}

						// Process is gone, but connection is still around -> dropped
						if( bProcessIsStillRunning == false )
						{
							Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Detected dropped local connection, cleaning up ({0:X8})", Local.Handle ) );
							DroppedConnections.Add( Local );
						}
						else
						{
							Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, String.Format( "[MaintainConnections] Local connection alive ({0:X8})", Local.Handle ) );
						}
					}
					else
					{
						RemoteConnection Remote = NextConnection as RemoteConnection;

						// Interface is no longer alive -> dropped
						if( !Remote.Interface.IsAlive() )
						{
							Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Detected dropped remote connection, cleaning up ({0:X8})", Remote.Handle ) );
							DroppedConnections.Add( Remote );
						}
						else
						{
							// If all children of a remote connection have been closed, then
							// we can ask this connection to close down as well, since its
							// services are no longer needed on this machine
							if( ( Remote.ChildrenSeen > 0 ) &&
								( Remote.LocalChildren.Count == 0 ) &&
								( Remote.RemoteChildren.Count == 0 ) )
							{
								Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Detected finished connection, closing ({0:X8})", Remote.Handle ) );
								Hashtable RemoteInParameters = null;
								Hashtable RemoteOutParameters = null;
								CloseConnection( Remote.Handle, RemoteInParameters, ref RemoteOutParameters );
							}
							else
							{
								Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, String.Format( "[MaintainConnections] Remote connection alive ({0:X8})", Remote.Handle ) );
							}
						}
					}
				}
				// For connections that are no longer active, clean up when we can
				else if( NextConnection.CurrentState == ConnectionState.DISCONNECTED )
				{
					if( NextConnection is LocalConnection )
					{
						LocalConnection Local = NextConnection as LocalConnection;

						// Connection is no longer active. If the local message queue has emptied,
						// add it to the list of connections to clean up. We only need to wait for
						// the local queue to empty here, since we know all messages have been
						// delived before we allowed the connection to be closed
						if( Local.MessageQueue.Count == 0 )
						{
							Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Local connection has closed ({0:X8})", Local.Handle ) );
							ClosedAndInactiveConnections.Add( Local );
						}
						else
						{
							// Wait for a couple seconds...
                            TimeSpan TimeSinceConnectionClosed = DateTime.UtcNow - Local.DisconnectedTime;
							TimeSpan TimeToWait = TimeSpan.FromSeconds(2);

							// If this one isn't going quietly and it's got a Job, wait until
							// the Job is gone at least (closed when the owner is closed, and
							// possibly killed in MaintainJobs). If it doesn't have a Job,
							// just add it to the list of connections to clean up.
							if( ( TimeSinceConnectionClosed > TimeToWait ) &&
								( Local.Job == null ) )
							{
								Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Local connection has timed-out and is being forceably closed ({0:X8})", Local.Handle ) );
								ClosedAndInactiveConnections.Add( Local );
							}
							else
							{
								Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, String.Format( "[MaintainConnections] Waiting for closed local conection to drain its message queue... ({0:X8})", Local.Handle ) );
							}
						}
					}
					else
					{
						Debug.Assert( NextConnection is RemoteConnection );

						// For any inactive remote connections, just remove them
						Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Remote connection has closed ({0:X8})", NextHandle ) );
						ClosedAndInactiveConnections.Add( NextConnection );
					}
				}
				else
				{
					Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Connection {0:X8} is {1}", NextHandle, NextConnection.CurrentState ) );
				}
			}

			// Close all dropped connections
			foreach( Connection Dropped in DroppedConnections )
			{
				// Only try to close the connection if it remains active.
				// Most common case of inactive here is an alias for a remote
				// connection that been closed in a previous iteration of this
				// same loop.
				if( Dropped.CurrentState == ConnectionState.CONNECTED )
				{
					Hashtable DroppedInParameters = null;
					Hashtable DroppedOutParameters = null;
					CloseConnection( Dropped.Handle, DroppedInParameters, ref DroppedOutParameters );

					// Different clean-up items depending on whether this is a remote or local connection
					if( Dropped is LocalConnection )
					{
						LocalConnection LocalClosed = Dropped as LocalConnection;

						// If a local connection was dropped, empty its message queue and signal
						// since we know all messages sent to this connection have been delivered
						// but will never be received
						lock( LocalClosed.MessageQueue )
						{
							LocalClosed.MessageQueue.Clear();
							LocalClosed.MessageAvailableSignal();
						}
					}
				}
			}

			// Remove all fully closed and inactive connections
			foreach( Connection Closed in ClosedAndInactiveConnections )
			{
				// For any closed connection, make sure all channels have been properly cleaned up
				foreach( Channel OpenChannel in Closed.OpenChannels.Values )
				{
					// If the file was opened for writing, discard the file completely
					// If the file was opened for reading, leave it alone
					if( ( OpenChannel.Flags & EChannelFlags.ACCESS_WRITE ) != 0 )
					{
						Log( EVerbosityLevel.Informative, ELogColour.Orange, "[MaintainConnections] Closing orphaned channel: \"" + OpenChannel.Name + "\"" );

						// Delete the file if it exists
						if( File.Exists( OpenChannel.FullName ) )
						{
							try
							{
								File.Delete( OpenChannel.FullName );
							}
							catch( Exception )
							{
								if( OpenChannel is JobChannel )
								{
									Log( EVerbosityLevel.Informative, ELogColour.Orange, "[MaintainConnections] Failed to delete orphaned, writable Job channel (generally benign): \"" + OpenChannel.Name + "\"" );
								}
								else
								{
									Log( EVerbosityLevel.Informative, ELogColour.Orange, "[MaintainConnections] Failed to delete orphaned, writable persistent cache channel (possible cache corruption): \"" + OpenChannel.Name + "\"" );
								}
							}
						}
					}
				}

				// Finally, remove the connection from the set
				if( Connections.Remove( Closed.Handle ) )
				{
					// If this is a remote connection, remove all known aliases first
					if( Closed is RemoteConnection )
					{
						RemoteConnection RemoteClosed = Closed as RemoteConnection;
						foreach( Int32 Alias in RemoteClosed.Aliases )
						{
							Connections.Remove( Alias );
							string LogMessage = String.Format( "[MaintainConnections] Removed alias for remote connection: {0:X8} was an alias for {1:X8}",
								Alias,
								RemoteClosed.Handle );
							Log( EVerbosityLevel.Informative, ELogColour.Green, LogMessage );
						}
					}

					Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[MaintainConnections] Removed connection {0:X8}", Closed.Handle ) );
				}

				// If this is the last connection open, log it out and reset the NextCleanUpCacheTime
				if( Connections.Count == 0 )
				{
					Log( EVerbosityLevel.Informative, ELogColour.Green, "[MaintainConnections] All connections have closed" );

					// Restart the agent's log on simple boundary conditions like this,
					// even though we may end up with more connections and jobs per file,
					// we won't end up spreading single connections across multiple logs
					AgentApplication.StartNewLogFile();

					// Update our state and ping the coordinator
					CurrentState = AgentState.Available;
					WorkingFor = "";
					PingCoordinator( true );

					// Wait a brief time before starting it
                    NextCleanUpCacheTime = DateTime.UtcNow + TimeSpan.FromSeconds(10);
				}
			}

			// If there are no more open connections, do any idle time adjustments
			if( Connections.Count == 0 )
			{
				// After all connections are closed, reset the coordinator condition
				CoordinatorResponding = true;
			}
		}

		private string GetProcessPathById(int ProcessID)
		{
			if (Environment.OSVersion.Platform == PlatformID.Unix)
			{ // Mac is "Unix" to Mono for compatiblity reasons
				return Path.GetFullPath(Path.GetDirectoryName(new Uri(Assembly.GetExecutingAssembly().CodeBase).AbsolutePath) + "/../Mac/");
			}

			var WMIQuery = string.Format("SELECT ExecutablePath FROM Win32_Process WHERE ProcessId = {0}", ProcessID);
			string ExecPath;

			using(var WMISearcher = new ManagementObjectSearcher(WMIQuery))
			using(var Results = WMISearcher.Get())
			{
				if (Results.Count == 0)
				{
					throw new InvalidOperationException(string.Format("Couldn't find the process with ID {0}.", ProcessID));
				}

				ExecPath = Results.Cast<ManagementObject>().First().GetPropertyValue("ExecutablePath") as string;
			}

			return Path.GetDirectoryName(ExecPath);
		}

	}
}

