// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
#if !__MonoCS__
using System.Deployment.Application;
#endif
using System.Diagnostics;
using System.Drawing;
using System.IO;
using System.Text;
using System.Threading;
using System.Windows.Forms;
using System.Runtime.InteropServices;
using System.Xml;
using System.Xml.Serialization;

using AgentInterface;
using UnrealControls;

namespace Agent
{
	public partial class SwarmAgentWindow : Form
	{
		public enum DialogFont
		{
			Consolas,
			Tahoma
		}

		/**
		 * Container class for a prioritised line of text
		 */
        public class LogLine
        {
            public EVerbosityLevel Verbosity;
            public ELogColour Colour;
            public string Line;

            public LogLine( EVerbosityLevel InVerbosity, ELogColour InColour, string InLine )
            {
                Verbosity = InVerbosity;
                Colour = InColour;
                Line = InLine;
            }
        }

        /**
         * Link up to the advanced (and quick) text box
         */
        private OutputWindowDocument MainLogDoc = new OutputWindowDocument();
        
		/*
		 * The container class for progression data
		 */
		private Progressions ProgressionData = null;

        /**
         * The main GUI window 
         */
		public SwarmAgentWindow()
		{
			InitializeComponent();

			this.BarToolTip = new System.Windows.Forms.ToolTip();
			this.BarToolTip.AutoPopDelay = 60000;
			this.BarToolTip.InitialDelay = 500;
			this.BarToolTip.ReshowDelay = 0;
			this.BarToolTip.ShowAlways = true;	// Force the ToolTip text to be displayed whether or not the form is active.

			AgentApplication.Options = ReadOptions<SettableOptions>( "SwarmAgent.Options.xml" );
			AgentApplication.Options.PostLoad();

			AgentApplication.DeveloperOptions = ReadOptions<SettableDeveloperOptions>( "SwarmAgent.DeveloperOptions.xml" );
			AgentApplication.DeveloperOptions.PostLoad();

			CreateBarColours( this );

			LogOutputWindowView.Document = MainLogDoc;
			SettingsPropertyGrid.SelectedObject = AgentApplication.Options;
			DeveloperSettingsPropertyGrid.SelectedObject = AgentApplication.DeveloperOptions;
		
			UpdateWindowState();

			// Set the title bar to include the name of the machine and the group
			TopLevelControl.Text = "Swarm Agent running on " + System.Net.Dns.GetHostName();

			LogOutputWindowView.Refresh();
		}

		public void SelectVisualizerTab()
		{
			AgentTabs.SelectedTab = VisualiserTab;
		}

		public void SaveOptions()
		{
			SaveWindowState();

			AgentApplication.Options.PreSave();
			WriteOptions<SettableOptions>( AgentApplication.Options, "SwarmAgent.Options.xml" );

			AgentApplication.DeveloperOptions.PreSave();
			WriteOptions<SettableDeveloperOptions>( AgentApplication.DeveloperOptions, "SwarmAgent.DeveloperOptions.xml" );
		}

        public void Destroy()
        {
			SaveOptions();
            Dispose();
        }

        /**
         * Default logger that output strings to the console and debug stream
         * with the timestamp added to the beginning of every message
         */
		delegate void DelegateLog( LogLine Line );
		
		public void Log (LogLine Line)
		{
			if (Line == null) {
				return;
			}

			// if we need to, invoke the delegate
			if (InvokeRequired) {
				Invoke (new DelegateLog (Log), new object[] { Line });
				return;
			}

			DateTime Now = DateTime.Now;
			string FullLine = Now.ToLongTimeString () + ": " + Line.Line;

			// translate the colour specified into an actual Color
			Color col;
			switch (Line.Colour) 
			{
				case ELogColour.Blue:
					col = Color.DarkBlue;
					break;
				case ELogColour.Orange:
					col = Color.Orange;
					break;
				case ELogColour.Red:
					col = Color.Red;
					break;
				default:
					col = Color.DarkGreen;
					break;
			}

            MainLogDoc.AppendLine( col, FullLine );

			Debug.WriteLineIf( System.Diagnostics.Debugger.IsAttached, FullLine );
        }

		delegate void DelegateClearLog();
		public void ClearLog()
		{
			if (IsHandleCreated)
			{
				if (InvokeRequired)
				{
					Invoke(new DelegateClearLog(ClearLog));
				}
				else
				{
					MainLogDoc.Clear();
				}
			}
		}

		/**
		 * Process a timing event received from Swarm
		 */
		delegate void DelegateProcessProgressionEvent( ProgressionEvent Event );

		public void ProcessProgressionEvent( ProgressionEvent Event )
		{
			if( Event == null )
			{
				return;
			}

			// if we need to, invoke the delegate
			if( InvokeRequired )
			{
				Invoke( new DelegateProcessProgressionEvent( ProcessProgressionEvent ), new object[] { Event } );
				return;
			}

			if( Event.State == EProgressionState.InstigatorConnected )
			{
				ProgressionData = new Progressions();
				OverallProgressBar.Invalidate();
			}

			if( ProgressionData != null )
			{
				if( ProgressionData.ProcessEvent( Event ) )
				{
					VisualiserGridViewResized = true;
					VisualiserGridView.Invalidate();
					OverallProgressBar.Invalidate();
				}
			}
		}

		public void Tick()
		{
			if( ProgressionData != null )
			{
				PopulateGridView();

				if( ProgressionData.Tick() )
				{
					VisualiserGridView.Invalidate();
				}
			}
		}

        protected void XmlSerializer_UnknownAttribute( object sender, XmlAttributeEventArgs e )
        {
        }

        protected void XmlSerializer_UnknownNode( object sender, XmlNodeEventArgs e )
        {
        }

        private T ReadOptions<T>( string OptionsFileName ) where T : new()
        {
            T Instance = new T();

            Stream XmlStream = null;
            try
            {
				string BaseDirectory;
#if !__MonoCS__
				if( ApplicationDeployment.IsNetworkDeployed )
				{
					ApplicationDeployment Deploy = ApplicationDeployment.CurrentDeployment;
					BaseDirectory = Deploy.DataDirectory;
				}
				else
#endif
                if (AgentApplication.OptionsFolder.Length > 0)
                {
                    // An options folder was specified on the command line
                    BaseDirectory = AgentApplication.OptionsFolder;
                }
                else
				{
					BaseDirectory = Application.StartupPath;
				}
				string FullPath = Path.Combine( BaseDirectory, OptionsFileName );

                // Get the XML data stream to read from
				XmlStream = new FileStream( FullPath, FileMode.Open, FileAccess.Read, FileShare.None, 256 * 1024, false );

                // Creates an instance of the XmlSerializer class so we can read the settings object
                XmlSerializer ObjSer = new XmlSerializer( typeof( T ) );
                // Add our callbacks for a busted XML file
                ObjSer.UnknownNode += new XmlNodeEventHandler( XmlSerializer_UnknownNode );
                ObjSer.UnknownAttribute += new XmlAttributeEventHandler( XmlSerializer_UnknownAttribute );

                // Create an object graph from the XML data
                Instance = ( T )ObjSer.Deserialize( XmlStream );
            }
            catch( Exception E )
            {
				Debug.WriteLineIf( System.Diagnostics.Debugger.IsAttached, E.Message );
            }
            finally
            {
                if( XmlStream != null )
                {
                    // Done with the file so close it
                    XmlStream.Close();
                }
            }

            return ( Instance );
        }

        private void WriteOptions<T>( T Data, string OptionsFileName )
        {
#if !__MonoCS__ // @todo Mac
			lock( Data )
#endif
            {
                Stream XmlStream = null;
                try
                {
					string BaseDirectory;
#if !__MonoCS__
					if( ApplicationDeployment.IsNetworkDeployed )
					{
						ApplicationDeployment Deploy = ApplicationDeployment.CurrentDeployment;
						BaseDirectory = Deploy.DataDirectory;
					}
					else
#endif
					if (AgentApplication.OptionsFolder.Length > 0)
                    {
                        // An options folder was specified on the command line
                        BaseDirectory = AgentApplication.OptionsFolder;
                    }
                    else
				    {
					    BaseDirectory = Application.StartupPath;
				    }
					string FullPath = Path.Combine( BaseDirectory, OptionsFileName );

					XmlStream = new FileStream( FullPath, FileMode.Create, FileAccess.Write, FileShare.None, 256 * 1024, false );
                    XmlSerializer ObjSer = new XmlSerializer( typeof( T ) );

                    // Add our callbacks for a busted XML file
                    ObjSer.UnknownNode += new XmlNodeEventHandler( XmlSerializer_UnknownNode );
                    ObjSer.UnknownAttribute += new XmlAttributeEventHandler( XmlSerializer_UnknownAttribute );

                    ObjSer.Serialize( XmlStream, Data );
                }
                catch( Exception E )
                {
					Debug.WriteLineIf( System.Diagnostics.Debugger.IsAttached, E.Message );
                }
                finally
                {
                    if( XmlStream != null )
                    {
                        // Done with the file so close it
                        XmlStream.Close();
                    }
                }
            }
        }

		private void UpdateFonts()
		{
			// Set the requested font
			LogOutputWindowView.Font = new Font( AgentApplication.Options.TextFont.ToString(), 9F );
			VisualiserGridView.Font = LogOutputWindowView.Font;
			SettingsPropertyGrid.Font = LogOutputWindowView.Font;

			LogOutputWindowView.Refresh();
		}

		private void SaveWindowState()
		{
			AgentApplication.Options.WindowLocation = Location;
			AgentApplication.Options.WindowSize = Size;
			AgentApplication.Options.AgentTabIndex = AgentTabs.SelectedIndex;
		}

		private void DeveloperMenuItemVisibilityChanged( Object Sender, EventArgs Args )
		{
			if( DeveloperMenuItem.Visible )
			{
				AgentTabs.TabPages.Insert( DeveloperSettingsTab.TabIndex, DeveloperSettingsTab );
			}
			else
			{
				AgentTabs.TabPages.RemoveAt( DeveloperSettingsTab.TabIndex );
			}
		}

		private void UpdateWindowState()
		{
			if( AgentApplication.Options.WindowLocation != new Point( 0, 0 ) )
			{
				Location = AgentApplication.Options.WindowLocation;
			}

			if( AgentApplication.Options.WindowSize != new Size( 0, 0 ) )
			{
				Size = AgentApplication.Options.WindowSize;
			}

			// Adjust the window location and size, if off-screen
			Rectangle VirtualScreenBounds = SystemInformation.VirtualScreen;

			Point WindowTL = AgentApplication.Options.WindowLocation;
			Point WindowBR = AgentApplication.Options.WindowLocation + AgentApplication.Options.WindowSize;
			WindowBR.X = Math.Min( WindowBR.X, VirtualScreenBounds.Right );
			WindowBR.Y = Math.Min( WindowBR.Y, VirtualScreenBounds.Bottom );
			WindowTL.X = Math.Max( WindowBR.X - AgentApplication.Options.WindowSize.Width, VirtualScreenBounds.Left );
			WindowTL.Y = Math.Max( WindowBR.Y - AgentApplication.Options.WindowSize.Height, VirtualScreenBounds.Top );

			Location = WindowTL;
			Size = new Size( WindowBR.X - WindowTL.X, WindowBR.Y - WindowTL.Y );

			// If the hidden developer menu should be shown, show it
			DeveloperMenuItem.VisibleChanged += new EventHandler( DeveloperMenuItemVisibilityChanged );
			DeveloperMenuItem.Visible = AgentApplication.Options.ShowDeveloperMenu;

			AgentTabs.SelectedIndex = AgentApplication.Options.AgentTabIndex;

			UpdateFonts();
		}

        private void ClickExitMenu( object sender, EventArgs e )
        {
            Hide();
            AgentApplication.RequestQuit();
        }

        private void ShowSwarmAgentWindow( object sender, MouseEventArgs e )
        {
			AgentApplication.ShowWindow = true;
        }

        private void SwarmAgentWindowClosing( object sender, FormClosingEventArgs e )
		{
			if (e.CloseReason == CloseReason.UserClosing)
			{
				e.Cancel = true;
				Hide();
			}
			else
			{
				AgentApplication.RequestQuit();
			}
		}

        private void CancelButtonClick( object sender, EventArgs e )
        {
            Hide();
        }

		private void EditClearClick( object sender, EventArgs e )
		{
			MainLogDoc.Clear();
		}

		private void MenuAboutClick( object sender, EventArgs e )
		{
			using( UnrealAboutBox About = new UnrealAboutBox( this.Icon, null ) )
			{
#if DEBUG
				About.Text = "About Swarm Agent (Debug Build)";
#else
				About.Text = "About Swarm Agent";
#endif
				About.ShowDialog( this );
			}
		}

		private void OptionsValueChanged( object s, PropertyValueChangedEventArgs e )
		{
			GridItem ChangedProperty = e.ChangedItem;
			if( ChangedProperty != null )
			{
				Type T = ChangedProperty.Value.GetType();
				if( T == typeof( SwarmAgentWindow.DialogFont ) )
				{
					UpdateFonts();
				}
				else if( T == typeof( Color ) )
				{
					CreateBarColours( this );
				}
				else if( ( ChangedProperty.Label == "EnableStandaloneMode" ) ||
						 ( ChangedProperty.Label == "AgentGroupName" ) )
				{
					AgentApplication.RequestPingCoordinator();
				}
				else if( ChangedProperty.Label == "CoordinatorRemotingHost" )
				{
					AgentApplication.RequestInitCoordinator();
				}
				else if( ChangedProperty.Label == "ShowDeveloperMenu" )
				{
					DeveloperMenuItem.Visible = ( bool )ChangedProperty.Value;
				}
				else if( ChangedProperty.Label == "CacheFolder" )
				{
					AgentApplication.RequestCacheRelocation();
				}

				// Always write out the latest options when anything changes
				SaveOptions();
			}
		}

		private void CacheClearClick( object sender, EventArgs e )
		{
			AgentApplication.RequestCacheClear();
		}

		private void CacheValidateClick( object sender, EventArgs e )
		{
			AgentApplication.RequestCacheValidation();
		}

		private void NetworkPingCoordinatorMenuItem_Click( object sender, EventArgs e )
		{
			if( AgentApplication.Options.EnableStandaloneMode )
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Not pinging coordinator, standalone mode enabled" );
			}
			else
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Pinging Coordinator..." );
				if( AgentApplication.RequestPingCoordinator() )
				{
					AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network]     Coordinator has responded normally" );
				}
				else
				{
					AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Orange, "[Network]     Coordinator has failed to respond" );
				}
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Coordinator ping complete" );
			}
		}

		private void NetworkPingRemoteAgentsMenuItem_Click( object sender, EventArgs e )
		{
			if( AgentApplication.Options.EnableStandaloneMode )
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Not pinging remote agents, standalone mode enabled" );
			}
			else
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Pinging remote agents..." );
				AgentApplication.RequestPingRemoteAgents();
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Remote Agent ping complete" );
			}
		}

		private void DeveloperRestartQAAgentsMenuItem_Click( object sender, EventArgs e )
		{
			if( AgentApplication.Options.EnableStandaloneMode )
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Not restarting QA agents, standalone mode enabled" );
			}
			else
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Restarting QA agents..." );
				AgentApplication.RequestRestartQAAgents();
			}
		}

		private void DeveloperRestartWorkerAgentsMenuItem_Click( object sender, EventArgs e )
		{
			if( AgentApplication.Options.EnableStandaloneMode )
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Not restarting worker agents, standalone mode enabled" );
			}
			else
			{
				AgentApplication.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Network] Restarting worker agents..." );
				AgentApplication.RequestRestartWorkerAgents();
			}
		}
	}

    static partial class AgentApplication
    {
#if !__MonoCS__
		[DllImport( "user32.dll" )]
		private static extern bool SetForegroundWindow( IntPtr hWnd );
#endif

        /**
         * The class containing all the client settable options
         * 
         * Serialised in on constuction of the window, out on destruction
         */
        public static SettableOptions Options = null;
		public static SettableDeveloperOptions DeveloperOptions = null;
        
        /**
         * Thread used to update the GUI
         */
        private static Thread ProcessGUIThread = null;

		/*
		 * Thread safe way of caching lines of the until the GUI thread is ready for them
		 */
        private static ReaderWriterQueue<SwarmAgentWindow.LogLine> LogLines = null;

		/*
		 * Thread safe way of caching lines of the until the GUI thread is ready for them
		 */
		private static ReaderWriterQueue<SwarmAgentWindow.ProgressionEvent> ProgressionEvents = null;

		/*
		 * Event to let the main thread know that the GUI thread is ready
		 */
        private static ManualResetEvent GUIInit = null;

		/*
		 * Set to false when an exit is requested
		 */
        private static bool Ticking = false;

		/*
		 * Set to true when the cache location has been modified
		 */
		private static bool CacheRelocationRequested = false;

		/*
		 * Set to true when a cache clear is requested
		 */
		private static bool CacheClearRequested = false;

		/*
		 * Set to true when a cache validate is requested
		 */
		private static bool CacheValidateRequested = false;

		/*
		 * Set to true when you want the window to pop up front and center
		 */
		public static bool ShowWindow = false;

        /*
		 * The folder that contains options for swarm. If empty, options will be located next to the executable.
		 */
        public static string OptionsFolder = "";

        /*
         * Variables private to the GUI thread
         */
        private static SwarmAgentWindow MainWindow = null;

		/*
		 * The current log file
		 */
		private static StreamWriter LogFile = null;

		/**
		 * A synchronization object for the log file.
		 */
		private static Object LogFileLock = new Object();

		public static void StartNewLogFile()
		{
			lock (LogFileLock)
			{
				// Close any existing stream first
				if (LogFile != null)
				{
					LogFile.Close();
					LogFile = null;
				}

				// Open a new log file marked by the UTC time
				string LogDirectory = Path.Combine(AgentApplication.Options.CacheFolder, "Logs");
				try
				{
					if (!Directory.Exists(LogDirectory))
					{
						// Create the directory
						Directory.CreateDirectory(LogDirectory);
					}
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Verbose, ELogColour.Red, "[StartNewLogFile] Error: " + Ex.Message);
				}

				if (Directory.Exists(LogDirectory))
				{
					string LogFileName = Path.Combine(LogDirectory, "AgentLog_" + DateTime.UtcNow.ToFileTimeUtc().ToString() + ".log");
					LogFile = new StreamWriter(LogFileName);
					LogFile.AutoFlush = true;
				}
			}
		}

		public static void ClearLogWindow()
		{
			if (MainWindow.InvokeRequired)
			{
				MainWindow.Invoke((MethodInvoker)(() => MainWindow.ClearLog()));
			}
			else
			{
				MainWindow.ClearLog();
			}			
		}

        private static void ProcessThreadQueues( TimeSpan MaximumExecutionTime )
        {
            DateTime TimeLimit = DateTime.UtcNow + MaximumExecutionTime;
            while (DateTime.UtcNow < TimeLimit)
			{
				// Just do one at a time
				if( ProgressionEvents.Count > 0 )
				{
					MainWindow.ProcessProgressionEvent( ProgressionEvents.Dequeue() );
				}
				// Handle all progression messages first, then just do one at a time
				else if( LogLines.Count > 0 )
				{
					MainWindow.Log( LogLines.Dequeue() );
				}
				else
				{
					break;
				}
			}
			MainWindow.Tick();
		}

		/**
         * Main GUI update thread
         */
		private static void ProcessGUIThreadProc()
        {
            MainWindow = new SwarmAgentWindow();

            LogLines = new ReaderWriterQueue<SwarmAgentWindow.LogLine>();
			ProgressionEvents = new ReaderWriterQueue<SwarmAgentWindow.ProgressionEvent>();

			StartNewLogFile();

            Ticking = true;
            GUIInit.Set();
            
			TimeSpan LoopIterationTime = TimeSpan.FromMilliseconds(100);
            while( Ticking )
            {
                Stopwatch SleepTimer = Stopwatch.StartNew();
				ProcessThreadQueues( LoopIterationTime );

				if( ShowWindow )
				{
#if !__MonoCS__
					SetForegroundWindow( MainWindow.Handle );
#endif
					MainWindow.SelectVisualizerTab();
					MainWindow.Show();
					ShowWindow = false;
				}

                Application.DoEvents();

                TimeSpan SleepTime = LoopIterationTime - SleepTimer.Elapsed;
				if( SleepTime.TotalMilliseconds > 0 )
				{
					Thread.Sleep( SleepTime );
				}
            }

            MainWindow.Destroy();
        }

		public static void Log( EVerbosityLevel Verbosity, ELogColour TextColour, string Line )
        {
			// Only consider the line if it's not the highest level of verbosity
			// unless the highest level of verbosity is what is being asked for
			if( ( EVerbosityLevel.SuperVerbose != Verbosity ) ||
				( EVerbosityLevel.SuperVerbose == AgentApplication.Options.Verbosity ) )
			{
				bool bShouldLogToConsole = Verbosity <= AgentApplication.Options.Verbosity;

				lock (LogFileLock)
				{
					// Log the line out to the file, if it exists
					if (LogFile != null)
					{
						LogFile.WriteLine(Line);
					}
					else
					{
						// If the file doesn't exist, always log to the console
						bShouldLogToConsole = true;
					}
				}

				if( bShouldLogToConsole )
				{
					LogLines.Enqueue( new SwarmAgentWindow.LogLine( Verbosity, TextColour, Line ) );
				}
			}
        }

		public static void UpdateMachineState( string Machine, int ThreadNum, EProgressionState NewState )
		{
			ProgressionEvents.Enqueue( new SwarmAgentWindow.ProgressionEvent( Machine, ThreadNum, NewState ) );
		}

        public static void RequestQuit()
        {
            Ticking = false;
        }

		public static void RequestCacheRelocation()
		{
			CacheRelocationRequested = true;
		}

		public static void RequestCacheClear()
		{
			CacheClearRequested = true;
		}

		public static void RequestCacheValidation()
		{
			CacheValidateRequested = true;
		}

		public static bool RequestPingCoordinator()
		{
			return LocalAgent.PingCoordinator( true );
		}

		public static void RequestInitCoordinator()
		{
			LocalAgent.InitCoordinator();
		}

		public static void RequestPingRemoteAgents()
		{
			LocalAgent.PingRemoteAgents( AgentApplication.Options.AllowedRemoteAgentGroup );
		}

		public static void RequestRestartQAAgents()
		{
			LocalAgent.RestartAgentGroup( "QATestGroup" );
		}

		public static void RequestRestartWorkerAgents()
		{
			LocalAgent.RestartAgentGroup( "DefaultDeployed" );
		}

        private static void ParseArgs(string[] args)
        {
            foreach (string arg in args)
            {
                if ( arg.StartsWith("-OptionsFolder=") )
                {
                    OptionsFolder = arg.Substring("-OptionsFolder=".Length);
                }
            }
        }

		private static void InitGUIThread()
        {
            GUIInit = new ManualResetEvent( false );

            ThreadStart ThreadStartProcessGUI = new ThreadStart( ProcessGUIThreadProc );
            ProcessGUIThread = new Thread( ThreadStartProcessGUI );
			ProcessGUIThread.Name = "ProcessGUIThread";
			ProcessGUIThread.SetApartmentState( ApartmentState.STA );
            ProcessGUIThread.Start();

            GUIInit.WaitOne();
        }
    }
}