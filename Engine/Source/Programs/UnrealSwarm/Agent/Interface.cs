// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.Threading;

using AgentInterface;

namespace Agent
{
	///////////////////////////////////////////////////////////////////////////

	/**
	 * External implementation of the agent interface processing and dispatching
	 */
	public partial class Agent : MarshalByRefObject, IAgentInternalInterface, IAgentInterface
	{
		public Int32 OpenConnection( Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( ( InParameters != null ) &&
				( InParameters.ContainsKey( "Version" ) ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "ProcessID" ) &&
							InParameters.ContainsKey( "LoggingFlags" ) )
						{
							return OpenConnection_1_0( InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 CloseConnection( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( ( InParameters != null ) &&
				( InParameters.ContainsKey( "Version" ) ) )
			{
				// Future use
			}
			else
			{
				return CloseConnection_1_0( ConnectionHandle, InParameters, ref OutParameters );
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 SendMessage( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "Message" ) )
						{
							return SendMessage_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 GetMessage( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "Timeout" ) )
						{
							return GetMessage_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 AddChannel( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "FullPath" ) &&
							InParameters.ContainsKey( "ChannelName" ) )
						{
							return AddChannel_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 TestChannel( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "ChannelName" ) )
						{
							return TestChannel_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 OpenChannel( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "ChannelName" ) &&
							InParameters.ContainsKey( "ChannelFlags" ) )
						{
							return OpenChannel_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 CloseChannel( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "ChannelHandle" ) )
						{
							return CloseChannel_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 OpenJob( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "JobGuid" ) )
						{
							return OpenJob_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 BeginJobSpecification( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( InParameters.ContainsKey( "Specification32" ) &&
							InParameters.ContainsKey( "Specification64" ) )
						{
							return BeginJobSpecification_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 AddTask( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( InParameters.ContainsKey( "Version" ) )
			{
				ESwarmVersionValue InParametersVersion = ( ESwarmVersionValue )InParameters["Version"];
				switch( InParametersVersion )
				{
					case ESwarmVersionValue.VER_1_0:
						if( ( InParameters.ContainsKey( "Specification" ) ) ||
							( InParameters.ContainsKey( "Specifications" ) ) )
						{
							return AddTask_1_0( ConnectionHandle, InParameters, ref OutParameters );
						}
					break;
				}
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 EndJobSpecification( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( ( InParameters != null ) &&
				( InParameters.ContainsKey( "Version" ) ) )
			{
				// Future use
			}
			else
			{
				return EndJobSpecification_1_0( ConnectionHandle, InParameters, ref OutParameters );
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 CloseJob( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Initialize the output parameters
			OutParameters = null;

			// Validate the input parameters
			if( ( InParameters != null ) &&
				( InParameters.ContainsKey( "Version" ) ) )
			{
				// Future use
			}
			else
			{
				return CloseJob_1_0( ConnectionHandle, InParameters, ref OutParameters );
			}

			return Constants.ERROR_INVALID_ARG;
		}

		public Int32 Method( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Currently, this method doesn't do anything
			return Constants.INVALID;
		}
	}
}

