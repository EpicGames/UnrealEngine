// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
using System.Data.SqlClient;
using System.Diagnostics;
using System.IO;
using System.Runtime.InteropServices;
using System.Security.Cryptography.X509Certificates;
using System.Threading;

using AgentInterface;
using SwarmCoordinatorInterface;

namespace Agent
{
	///////////////////////////////////////////////////////////////////////////

	public class AgentTask
	{
		public AgentTask( AgentTaskSpecification NewSpecification )
		{
			Specification = NewSpecification;
			CurrentState = new AgentTaskState( NewSpecification.JobGuid, NewSpecification.TaskGuid, EJobTaskState.TASK_STATE_IDLE );
			CurrentOwner = null;
		}

        public static Int32 CompareTasksByCostDescending(AgentTask A, AgentTask B)
		{
			if( ( A.Specification != null ) &&
				( B.Specification != null ) )
			{
				return -A.Specification.Cost.CompareTo( B.Specification.Cost );
			}
			return 0;
		}

		public static Int32 CompareTasksByAssignTime( AgentTask A, AgentTask B )
		{
			return A.AssignTime.CompareTo( B.AssignTime );
		}

		public AgentTaskSpecification	Specification;
		public AgentTaskState			CurrentState;
		public Connection				CurrentOwner;
		public DateTime					AssignTime;
		public DateTime					StartTime;
		public DateTime					StopTime;
	}

	public class AgentJob
	{
		public enum JobState
		{
			AGENT_JOB_UNSPECIFIED,	// Jobs start in this state
			AGENT_JOB_PENDING,		// Once BeginJobSpecification is called
			AGENT_JOB_RUNNING,		// Once EndJobSpecification is called
			AGENT_JOB_CLOSED,		// Once CloseJob is called
		}

		public enum JobSuccessState
		{
			AGENT_JOB_INCOMPLETE,	// Jobs start in this state
			AGENT_JOB_SUCCESS,		// Once CloseJob is called, if the Job was a success
			AGENT_JOB_FAILURE,		// Once CloseJob is called, if the Job was a failure
		}

		public AgentJob( Agent NewManager )
		{
			// Everything else is already initialized to defaults
			Manager = NewManager;
		}

		public Int32 OpenJob( Connection NewOwner, AgentGuid NewJobGuid )
		{
			// Create the bi-directional link between the job and its owner
			NewOwner.Job = this;
			Owner = NewOwner;
			
			// Determine if the owner is the Instigator, and if so, perform
			// any additional work necessary
			if( NewOwner is LocalConnection )
			{
				OwnerIsInstigator = true;

				// If the owner of the job is the Instigator, set the current working
				// directory to be where the Instigator process is executing
				LocalConnection LocalNewOwner = NewOwner as LocalConnection;
				Manager.SetCurrentDirectoryByProcessID( LocalNewOwner.ProcessID );

				// Update the visualizer
				AgentApplication.UpdateMachineState(System.Net.Dns.GetHostName(), -1, EProgressionState.InstigatorConnected );
			}

			// Add to the Agent-wide list of active jobs
			JobGuid = NewJobGuid;
			Manager.ActiveJobs.Add( NewJobGuid, this );

			return Constants.SUCCESS;
		}

		private bool RequestDependency(Connection RequestingConnection, string Dependency, bool bIsRequired)
		{
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["ChannelName"] = Dependency;
			Hashtable OutParameters = null;
			if (Manager.TestChannel(RequestingConnection.Handle, InParameters, ref OutParameters) < 0)
			{
				if (RequestingConnection is RemoteConnection)
				{
					// For remote connection misses, try to pull the file from the instigator
					Manager.Log(EVerbosityLevel.Verbose, ELogColour.Green, "[Job] Attempting to pull file from remote Agent: " + Dependency);
					if (Manager.PullChannel(RequestingConnection as RemoteConnection, Dependency, null, 5) == false)
					{
						if (bIsRequired)
						{
							Manager.Log(EVerbosityLevel.Informative, ELogColour.Red, "[Job] Error: Failed to pull a required file from remote Agent: " + Dependency);
							return false;
						}
						else
						{
							Manager.Log(EVerbosityLevel.Verbose, ELogColour.Orange, "[Job] Warning: Failed to pull an optional file from remote Agent: " + Dependency);
						}
					}
				}
				else
				{
					Debug.Assert(RequestingConnection is LocalConnection);

					if (bIsRequired)
					{
						// Always fail on local connection misses
						Manager.Log(EVerbosityLevel.Informative, ELogColour.Red, "[Job] Error: Failed to find required file: " + Dependency);
						return false;
					}
					else
					{
						Manager.Log(EVerbosityLevel.Verbose, ELogColour.Orange, "[Job] Warning: Failed to find an optional file: " + Dependency);
					}
				}
			}
			return true;
		}

		private bool RequestJobFiles( Connection RequestingConnection, AgentJobSpecification JobSpecification )
		{
			// Check for and possibly request the executable
			if( !RequestDependency( RequestingConnection, JobSpecification.ExecutableName, true ) )
			{
				return false;
			}
			// Check for and possibly request each dependency
			if( JobSpecification.RequiredDependencies != null )
			{
				foreach( string Dependency in JobSpecification.RequiredDependencies )
				{
					if( !RequestDependency( RequestingConnection, Dependency, true ) )
					{
						return false;
					}
				}
			}
			if( JobSpecification.OptionalDependencies != null )
			{
				foreach( string Dependency in JobSpecification.OptionalDependencies )
				{
					if( !RequestDependency( RequestingConnection, Dependency, false ) )
					{
						return false;
					}
				}
			}
			return true;
		}

#if !__MonoCS__
		[DllImport("kernel32.dll", SetLastError = true, CallingConvention = CallingConvention.Winapi)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool IsWow64Process([In] IntPtr hProcess, [Out] out bool lpSystemInfo);
#endif

		public bool Allow64bitJobs()
		{
#if !__MonoCS__
			bool bIsUsing64bitOperatingSystem = false;

			// 64-bit process running in 64-bit Windows
			if ( IntPtr.Size == 8 )
			{
				bIsUsing64bitOperatingSystem = true;
			}
			// 32-bit process
			else if ( IntPtr.Size == 4 )
			{
				// Check if we're using WOW64 to run on 64-bit Windows.
				if ( IsWow64Process(Process.GetCurrentProcess().Handle, out bIsUsing64bitOperatingSystem) == false )
				{
					// Got some error, assume we're not running on 64-bit Windows.
					bIsUsing64bitOperatingSystem = false;
				}
			}
			return bIsUsing64bitOperatingSystem;
#else
			return true;
#endif
		}

		public Int32 BeginJobSpecification( AgentJobSpecification NewJobSpecification32, Hashtable NewJobDescription32, AgentJobSpecification NewJobSpecification64, Hashtable NewJobDescription64 )
		{
			Int32 ErrorCode = Constants.INVALID;

			bool bAllow64bitJobs = Allow64bitJobs();
			bool bUse64bitJob = (bAllow64bitJobs && NewJobSpecification64 != null) ? true : false;

			// Using the provided specifications, request all necessary Job files. For local
			// Agents, the executable and dependencies must already exist in the cache. For
			// remote Agents, we'll need request the necessary files from the Instigator.
			bool bReceivedJobFiles = false;

			// Am I the instigator?
			if ( Owner is LocalConnection )
			{
				// Request the files for both job specifications (so we can hand out either version to remote agents).
				if ( NewJobSpecification32 != null )
				{
					bReceivedJobFiles = RequestJobFiles( Owner, NewJobSpecification32 );
				}
				if ( NewJobSpecification64 != null && (NewJobSpecification32 == null || bReceivedJobFiles) )
				{
					bReceivedJobFiles = RequestJobFiles( Owner, NewJobSpecification64 );
				}
			}
			else
			{
				// Request the files for my job specification only.
				bReceivedJobFiles = RequestJobFiles( Owner, bUse64bitJob ? NewJobSpecification64 : NewJobSpecification32 );
			}

			if ( bReceivedJobFiles )
			{
				// If this succeeds, set the new specification and update the job state
				Specification32 = NewJobSpecification32;
				Specification64 = NewJobSpecification64;
				Description32 = NewJobDescription32;
				Description64 = NewJobDescription64;

				Specification = bUse64bitJob ? NewJobSpecification64 : NewJobSpecification32;
				Description = bUse64bitJob ? NewJobDescription64 : NewJobDescription32;

				CurrentState = JobState.AGENT_JOB_PENDING;

				ErrorCode = Constants.SUCCESS;
			}
			else
			{
				Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] BeginJobSpecification: Failed to cache necessary job files" );
				ErrorCode = Constants.ERROR_CHANNEL_NOT_FOUND;
			}

			return ( ErrorCode );
		}

		private bool RequestTaskFiles( Connection RequestingConnection, AgentTaskSpecification TaskSpecification )
		{
			// Check for and possibly request each dependency
			if( TaskSpecification.Dependencies != null )
			{
				foreach( string Dependency in TaskSpecification.Dependencies )
				{
					if( !RequestDependency( RequestingConnection, Dependency, true ) )
					{
						return false;
					}
				}
			}
			return true;
		}

		public Int32 AddTask( AgentTaskSpecification NewTaskSpecification )
		{
			Int32 ErrorCode = Constants.INVALID;

			// Using the provided specification, request all necessary Task files. For local
			// Agents, the executable and dependencies must already exist in the cache. For
			// remote Agents, we'll need request the necessary files from the Instigator.
			if( RequestTaskFiles( Owner, NewTaskSpecification ) )
			{
				lock( PendingTasks )
				{
					Manager.Log( EVerbosityLevel.Verbose, ELogColour.Green, "[Job] AddTask: Adding \"" + NewTaskSpecification.Parameters + "\" (" + NewTaskSpecification.TaskGuid.ToString() + ") with cost " + NewTaskSpecification.Cost.ToString() );

					// Queue the Task
					PendingTasks.Push( new AgentTask( NewTaskSpecification ) );
					TaskCount++;

					// Sanity check a couple assumptions here
					if( PendingTasks.Count > 1 )
					{
						Debug.Assert( TaskReservationCount == 0 );
					}
					if( TaskReservationCount > 0 )
					{
						Debug.Assert( PendingTasks.Count == 1 );
					}
				}

				// After adding a new Task, check for any outstanding reservations
				CheckForReservations();

				// Done
				ErrorCode = Constants.SUCCESS;
			}
			else
			{
				Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] AddTask: Failed to cache necessary task files" );
				ErrorCode = Constants.ERROR_CHANNEL_NOT_FOUND;
			}

			return ( ErrorCode );
		}

		private void SendJobCompletedMessage( AgentInfoMessage AdditionalInfoMessage )
		{
			// Only send a message back if we're the Instigator - remote workers don't have
			// enough information to make this determination
			Debug.Assert( OwnerIsInstigator );

			AgentJobState UpdatedStateMessage = null;
			if( CurrentSuccessState == JobSuccessState.AGENT_JOB_SUCCESS )
			{
				Manager.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Job] " + AdditionalInfoMessage.TextMessage );
				UpdatedStateMessage = new AgentJobState( JobGuid, EJobTaskState.STATE_COMPLETE_SUCCESS );
			}
			else if( CurrentSuccessState == JobSuccessState.AGENT_JOB_FAILURE )
			{
				Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, "[Job] " + AdditionalInfoMessage.TextMessage );
				UpdatedStateMessage = new AgentJobState( JobGuid, EJobTaskState.STATE_COMPLETE_FAILURE );
			}

			// Only if we have an actual update should we send one
			if( UpdatedStateMessage != null )
			{
				// Set the running time
				TimeSpan JobRunningTime = DateTime.UtcNow - StartTime;
				UpdatedStateMessage.JobRunningTime = JobRunningTime.TotalSeconds;

				// Set the exit code, if there is one
				UpdatedStateMessage.JobExitCode = ProcessObjectExitCode;

				// Send the actual message and the additional state message
				if( AdditionalInfoMessage != null )
				{
					Manager.SendMessageInternal( Owner, AdditionalInfoMessage );
				}
				Manager.SendMessageInternal( Owner, UpdatedStateMessage );
			}
		}

		public void OutputReceivedDataEventHandler( Object Sender, DataReceivedEventArgs Line )
		{
			if( ( Line != null ) &&
				( Line.Data != null ) )
			{
				if( ProcessObjectOutputLines < AgentApplication.DeveloperOptions.MaximumJobApplicationLogLines )
				{
					Manager.Log( EVerbosityLevel.Informative, ELogColour.Blue, Line.Data );
				}
				else
				{
					if( ProcessObjectOutputLines == AgentApplication.DeveloperOptions.MaximumJobApplicationLogLines )
					{
						Manager.Log( EVerbosityLevel.Informative, ELogColour.Blue, "Maximum output from job application received (defined in the Agent Settings)" );
						Manager.Log( EVerbosityLevel.Informative, ELogColour.Blue, "Remaining output from job application truncated - see the job log for complete output" );
					}
			
					// Send it along anyway, but mark it as the highest level of verbosity
					Manager.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Blue, Line.Data );
				}
				ProcessObjectOutputLines++;
			}
		}

		public void ErrorReceivedDataEventHandler( Object Sender, DataReceivedEventArgs Line )
		{
			if( ( Line != null ) &&
				( Line.Data != null ) )
			{
				if( ProcessObjectOutputLines < AgentApplication.DeveloperOptions.MaximumJobApplicationLogLines )
				{
					Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, Line.Data );
				}
				else
				{
					if( ProcessObjectOutputLines == AgentApplication.DeveloperOptions.MaximumJobApplicationLogLines )
					{
						Manager.Log( EVerbosityLevel.Informative, ELogColour.Blue, "Maximum output from job application received (defined in the Agent Settings)" );
						Manager.Log( EVerbosityLevel.Informative, ELogColour.Blue, "Remaining output from job application truncated - see the job log for complete output" );
					}

					// Send it along anyway, but mark it as the highest level of verbosity
					Manager.Log( EVerbosityLevel.ExtraVerbose, ELogColour.Red, Line.Data );
				}
				ProcessObjectOutputLines++;
			}
		}

		private Object PostJobStatsToDBLock = new Object();
		private void PostJobStatsToDB()
		{
			try
			{
				bool ShouldUpdateDB = false;
				lock( CurrentStateLock )
				{
					// If the job is closed and the process has exited, try to update the DB
					if( ( CurrentState == JobState.AGENT_JOB_CLOSED ) &&
						( ProcessObject.HasExited == true ) )
					{
						ShouldUpdateDB = true;
					}
				}

				lock( PostJobStatsToDBLock )
				{
					// Only try to connect once and only if DB name is not empty
					if( ( ShouldUpdateDB ) &&
						( JobStatsPostedToDB == false ) &&
						( Properties.Settings.Default.StatsDatabaseName.Length > 0 ) )
					{
						// Build query string using build in Windows Authentication and lowers the connection timeout to 3 seconds
						string ConnectionString = String.Format( "Data Source={0};Initial Catalog=SwarmStats;Trusted_Connection=Yes;Connection Timeout=3;", Properties.Settings.Default.StatsDatabaseName );
						SqlConnection PerfDBConnection = new SqlConnection( ConnectionString );

						// Open connection to DB
						PerfDBConnection.Open();

						// WARNING: DO NOT MODIFY THE BELOW WITHOUT UPDATING THE DB, BUMPING THE VERSION AND REGENERATE THE CREATE SCRIPTS

						// Create command string for stored procedure
						string SqlCommandString = "EXEC dbo.AddJob_v1";
						SqlCommandString += String.Format( " @Duration='{0}',", ( StopTime - StartTime ).TotalSeconds );
						SqlCommandString += String.Format( " @UserName='{0}',", Environment.UserName.ToUpperInvariant() );
						SqlCommandString += String.Format( " @MachineName='{0}',", System.Net.Dns.GetHostName().ToUpperInvariant() );
						SqlCommandString += String.Format( " @GroupName='{0}',", AgentApplication.Options.AgentGroupName );
						SqlCommandString += String.Format( " @JobGUID='{0}',", JobGuid.ToString() );
						SqlCommandString += String.Format( " @Instigator='{0}',", OwnerIsInstigator );
						SqlCommandString += String.Format( " @DistributionEnabled='{0}',", !AgentApplication.Options.EnableStandaloneMode );
						SqlCommandString += String.Format( " @RemoteMachineCount='{0}',", Owner.RemoteChildrenSeen );
						SqlCommandString += String.Format( " @ExecutableName='{0}',", Specification.ExecutableName );
						SqlCommandString += String.Format( " @Parameters='{0}',", Specification.Parameters );
						SqlCommandString += String.Format( " @JobWasSuccess='{0}',", ( CurrentSuccessState == JobSuccessState.AGENT_JOB_SUCCESS ) );
						SqlCommandString += String.Format( " @TotalTaskCount='{0}',", TaskCount );
						SqlCommandString += String.Format( " @DistributedTaskCount='{0}',", TaskCountRemote );
						SqlCommandString += String.Format( " @RestartedTaskCount='{0}',", TaskCountRequeue );
						SqlCommandString += String.Format( " @CacheHitRate='{0}',", ( CacheRequests > 0 ) ? ( ( ( float )CacheHits / ( float )CacheRequests ) * 100.0f ) : 0.0f );
						SqlCommandString += String.Format( " @NetworkBytesSent='{0}',", NetworkBytesSent );
						SqlCommandString += String.Format( " @NetworkBytesReceived='{0}',", NetworkBytesReceived );
						SqlCommandString += String.Format( " @PeakVirtualMemoryUsed='{0}',", ProcessObjectPeakVirtualMemorySize64 );
						SqlCommandString += String.Format( " @PeakPhysicalMemoryUsed='{0}',", ProcessObjectPeakWorkingSet64 );
						SqlCommandString += String.Format( " @TotalProcessorTime='{0}',", ProcessObject.TotalProcessorTime.TotalSeconds );
						SqlCommandString += String.Format( " @ExitCode='{0}',", ProcessObjectExitCode );
						SqlCommandString += String.Format( " @QualityName='{0}',", ( Description.Contains( "QualityLevel" ) ? Description["QualityLevel"] as string : "" ) );
						SqlCommandString += String.Format( " @GameName='{0}',", ( Description.Contains( "GameName" ) ? Description["GameName"] as string : "" ) );
						SqlCommandString += String.Format( " @MapName='{0}'", ( Description.Contains( "MapName" ) ? Description["MapName"] as string : "" ) );

						// Execute stored procedure adding build summary information
						Manager.Log( EVerbosityLevel.Verbose, ELogColour.Green, "[PostJobStatsToDB] Posting stats to DB" );
						Manager.Log( EVerbosityLevel.Verbose, ELogColour.Green, "[PostJobStatsToDB] " + SqlCommandString );
						SqlCommand SendSummaryCommand = new SqlCommand( SqlCommandString, PerfDBConnection );
						SendSummaryCommand.ExecuteNonQuery();

						// We're done, close the connection
						PerfDBConnection.Close();

						JobStatsPostedToDB = true;
					}
				}
			}
			// Catch exceptions here instead of at higher level as we don't care if DB connection fails.
			catch( Exception Ex )
			{
				Manager.Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[PostJobStatsToDB] Database error:" );
				Manager.Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[PostJobStatsToDB] " + Ex.Message );
			}
		}

		public void ExitedProcessEventHandler( Object Sender, EventArgs Args )
		{
			// Verify that the process is the one we think it is and update the Job state
			if( ProcessObject == ( Sender as Process ) )
			{
				Debug.Assert( ProcessObject.HasExited );

				// Grab any additional data from the ProcessObject before we let it go
				ProcessObjectExitCode = ProcessObject.ExitCode;

				lock( CurrentSuccessStateLock )
				{
					// Only update if the job state has not been determined by other means
					if( CurrentSuccessState == JobSuccessState.AGENT_JOB_INCOMPLETE )
					{
						// Determine if this is an agent managed process
						bool IsAnAgentManagedProcess =
							( Specification.JobFlags & EJobTaskFlags.FLAG_MANUAL_START ) == 0;

						// If this is an agent managed, Task-based process
						if( ( IsAnAgentManagedProcess ) &&
							( TaskCount > 0 ) )
						{
							// If the Job executable didn't close cleanly it's marked a failure,
							// otherwise, we'll wait until CloseJob is called to determine if
							// it's a success
							if( ProcessObject.ExitCode != 0 )
							{
								CurrentSuccessState = JobSuccessState.AGENT_JOB_FAILURE;
								if( OwnerIsInstigator )
								{
									// Log and send an INFO message describing the failure
									string NewMessageText = "Job has failed! Job executable didn't exit cleanly. Exit code: " + ProcessObject.ExitCode.ToString();
									SendJobCompletedMessage( new AgentInfoMessage( NewMessageText ) );
								}
							}
						}
						else
						{
							// Otherwise, this is an Agent managed, non-Task-based Job, or this Job
							// is manually managed outside of the Agent. In either case, the exit
							// code will determine success.
							if( ProcessObject.ExitCode == 0 )
							{
								CurrentSuccessState = JobSuccessState.AGENT_JOB_SUCCESS;
								if( OwnerIsInstigator )
								{
									// Log and send an INFO message describing the success
									string NewMessageText = "Job is a success!";
									SendJobCompletedMessage( new AgentInfoMessage( NewMessageText ) );
								}
							}
							else
							{
								CurrentSuccessState = JobSuccessState.AGENT_JOB_FAILURE;
								if( OwnerIsInstigator )
								{
									// Log and send an INFO message describing the failure
									string NewMessageText = "Job has failed! Job executable has exited with a non-zero exit code";
									SendJobCompletedMessage( new AgentInfoMessage( NewMessageText ) );
								}
							}
						}
					}
				}

				// Attempt to report the final stats to the DB
				PostJobStatsToDB();
			}
		}

		private Int32 StartJobExecution()
		{
			Int32 ErrorCode = Constants.INVALID;
			try
			{
				string JobsFolder = Path.Combine( AgentApplication.Options.CacheFolder, "Jobs" );
				string JobSpecificFolder = Path.Combine( JobsFolder, "Job-" + JobGuid.ToString() );

				// Assert that the folders we need are there
				Debug.Assert( Directory.Exists( JobsFolder ) &&
							  Directory.Exists( JobSpecificFolder ) );

				// Copy all required files into the Job directory
				foreach( KeyValuePair<string, string> Pair in Specification.DependenciesOriginalNames )
				{
					string SrcFile = Path.Combine( AgentApplication.Options.CacheFolder, Pair.Key );
					string DestFile = Path.Combine( JobSpecificFolder, Pair.Value );

					// Before the file is copied into the job directory, security check
					bool bIsAuthorized = true;
					bool bNeedToCheckFile = ( Path.GetExtension( SrcFile ) == ".exe" ) ||
											( Path.GetExtension( SrcFile ) == ".dll" );

					// Microsoft redistributable binaries won't be signed by Epic, so allow them to slide
					if( Path.GetFileName( SrcFile ).StartsWith( "msvc", StringComparison.InvariantCultureIgnoreCase ) )
					{
						bNeedToCheckFile = false;
					}

					if( ( bNeedToCheckFile ) &&
						( Manager.Certificate != null ) )
					{
						// If the Agent is signed, then everything else must be signed as well.
						// Start off pessimistic
						bIsAuthorized = false;

						X509Certificate NextCertificate = null;
						try
						{
							NextCertificate = X509Certificate.CreateFromSignedFile( SrcFile );
						}
						catch( Exception )
						{
							// Any exception means that either the file isn't signed or has an invalid certificate
						}
						if( NextCertificate != null )
						{
							if( NextCertificate.Equals( Manager.Certificate ) )
							{
								bIsAuthorized = true;
							}
						}
					}
					if( bIsAuthorized )
					{
						File.Copy( SrcFile, DestFile, true );
					}
					else
					{
						Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, "[Job] Failed to use file \"" + Pair.Key + "\" because of a security violation" );
					}
				}

				// Used to indicate the Job was started successfully
				bool JobStartedSuccessfully = false;

				// First determine if we're even supposed to start the executable
				EJobTaskFlags JobFlags = Specification.JobFlags;
				if( ( JobFlags & EJobTaskFlags.FLAG_MANUAL_START ) != 0 )
				{
					// User has asked to start the executable on their own (useful
					// for debugging). At connection time, we'll try to match the
					// full executable path name to establish the parent
					// connection. Simply update the state of the Job.
					JobStartedSuccessfully = true;
				}
				else
				{
					string OriginalExecutableName;
					if( Specification.DependenciesOriginalNames.TryGetValue( Specification.ExecutableName, out OriginalExecutableName ) )
					{
						string FullExecutableName = Path.Combine( JobSpecificFolder, OriginalExecutableName );

						Process TaskProcess = new Process();
						TaskProcess.StartInfo.FileName = FullExecutableName;
						TaskProcess.StartInfo.Arguments = Specification.Parameters;
						TaskProcess.StartInfo.WorkingDirectory = JobSpecificFolder;
						TaskProcess.StartInfo.CreateNoWindow = true;

						// Other properties worth looking into setting
						//   TaskProcess.StartInfo.RedirectStandardInput

						// Set up the redirect for output
						TaskProcess.StartInfo.UseShellExecute = false;
						TaskProcess.StartInfo.RedirectStandardOutput = true;
						TaskProcess.StartInfo.RedirectStandardError = true;
						TaskProcess.OutputDataReceived += new DataReceivedEventHandler( OutputReceivedDataEventHandler );
						TaskProcess.ErrorDataReceived += new DataReceivedEventHandler( ErrorReceivedDataEventHandler );

						// If not already set, set the number of allowed cores to use,
						// which should be respected by swarm-aware applications
						if( OwnerIsInstigator )
						{
							// Use local settings, if it's not already set
							if( !TaskProcess.StartInfo.EnvironmentVariables.ContainsKey( "Swarm_MaxCores" ) )
							{
								TaskProcess.StartInfo.EnvironmentVariables.Add( "Swarm_MaxCores", AgentApplication.DeveloperOptions.LocalJobsDefaultProcessorCount.ToString() );
							}
						}
						else
						{
							// Use remote settings
							TaskProcess.StartInfo.EnvironmentVariables.Add( "Swarm_MaxCores", AgentApplication.DeveloperOptions.RemoteJobsDefaultProcessorCount.ToString() );
						}

						// Set up our exited callback
						TaskProcess.Exited += new EventHandler( ExitedProcessEventHandler );
						TaskProcess.EnableRaisingEvents = true;

						ProcessObject = TaskProcess;
						if( TaskProcess.Start() )
						{
							if( OwnerIsInstigator )
							{
								if( AgentApplication.Options.AvoidLocalExecution )
								{
									// If we're avoiding local execution, make it as non-invasive as possible
									TaskProcess.PriorityClass = ProcessPriorityClass.Idle;
								}
								else
								{
									// Use local settings
									TaskProcess.PriorityClass = ( ProcessPriorityClass )AgentApplication.DeveloperOptions.LocalJobsDefaultProcessPriority;
								}
							}
							else
							{
								// Use remote settings
								TaskProcess.PriorityClass = ( ProcessPriorityClass )AgentApplication.DeveloperOptions.RemoteJobsDefaultProcessPriority;
							}
							TaskProcess.BeginOutputReadLine();
							TaskProcess.BeginErrorReadLine();

							Manager.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Job] Launched Job " + Specification.ExecutableName );
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Job]     PID is " + ProcessObject.Id.ToString() );
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Job]     GUID is \"" + JobGuid.ToString() + "\"" );

							// Success
							JobStartedSuccessfully = true;
						}
					}
					else
					{
						//@TODO: Error handling...
					}
				}

				// If the job was started successfully, however that may be, update
				// the state of the job
				if( JobStartedSuccessfully )
				{
					CurrentState = JobState.AGENT_JOB_RUNNING;
					StartTime = DateTime.UtcNow;
					ErrorCode = Constants.SUCCESS;
				}
				
				// Send a message indicating the Job has started, if this is the original owner
				if( ( OwnerIsInstigator ) &&
					( CurrentState == JobState.AGENT_JOB_RUNNING ) )
				{
					AgentJobState JobStartedMessage = new AgentJobState( JobGuid, EJobTaskState.STATE_RUNNING );
					Manager.SendMessageInternal( Owner, JobStartedMessage );

					// Also, if enabled, request that the agent window be brought
					// to the front, but only for the Instigator
					if (AgentApplication.Options.BringToFront && ((JobFlags & EJobTaskFlags.FLAG_MINIMIZED) == 0))
					{
						AgentApplication.ShowWindow = true;
					}
				}
			}
			catch( Exception Ex )
			{
				// Be sure to null out the process object if anything went wrong (usually starting the process)
				ProcessObject = null;

				Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[StartJobExecution] Exception was: " + Ex.ToString() );
				ErrorCode = Constants.ERROR_EXCEPTION;
			}

			return ( ErrorCode );
		}
		
		public class OpenJobOnRemoteData
		{
			public OpenJobOnRemoteData( AgentJob NewJob, RemoteConnection NewRemote )
			{
				Job = NewJob;
				Remote = NewRemote;
			}

			public AgentJob			Job;
			public RemoteConnection	Remote;
		}

		static void OpenJobOnRemoteThreadProc( Object ObjectState )
		{
			OpenJobOnRemoteData MethodData = ObjectState as OpenJobOnRemoteData;
			if( MethodData != null )
			{
				AgentJob Job = MethodData.Job;
				RemoteConnection Remote = MethodData.Remote;

				Hashtable OpenJobInParameters = new Hashtable();
				OpenJobInParameters["Version"] = ESwarmVersionValue.VER_1_0;
				OpenJobInParameters["JobGuid"] = Job.JobGuid;
				Hashtable OpenJobOutParameters = null;

				Int32 JobErrorCode = Remote.Interface.OpenJob( Remote.Handle, OpenJobInParameters, ref OpenJobOutParameters );
				if( JobErrorCode >= 0 )
				{
					Hashtable BeginInParameters = new Hashtable();
					BeginInParameters["Version"] = ESwarmVersionValue.VER_1_0;
					BeginInParameters["Specification32"] = Job.Specification32;
					BeginInParameters["Specification64"] = Job.Specification64;
					BeginInParameters["Description32"] = Job.Description32;
					BeginInParameters["Description64"] = Job.Description64;
					Hashtable BeginOutParameters = null;

					JobErrorCode = Remote.Interface.BeginJobSpecification( Remote.Handle, BeginInParameters, ref BeginOutParameters );
					if( JobErrorCode >= 0 )
					{
						// If the job is in deterministic mode, add all this agent's tasks now
						// so that if, for any reason, this connection is lost or fails, then
						// all remaining tasks assigned here will fail
						if( Job.DeterministicModeEnabled )
						{
							AgentTaskRequestResponse Response = Job.GetNextTask( Remote );
							while( ( Response != null ) &&
								   ( Response is AgentTaskSpecification ) )
							{
								Hashtable AddTaskInParameters = new Hashtable();
								AddTaskInParameters["Version"] = ESwarmVersionValue.VER_1_0;
								AddTaskInParameters["Specification"] = Response as AgentTaskSpecification;
								Hashtable AddTaskOutParameters = null;

								JobErrorCode = Remote.Interface.AddTask( Remote.Handle, AddTaskInParameters, ref AddTaskOutParameters );
								Response = null;
								if( JobErrorCode >= 0 )
								{
									Response = Job.GetNextTask( Remote );
								}
							}
						}
					}
					if( JobErrorCode >= 0 )
					{
						Hashtable EndJobInParameters = null;
						Hashtable EndJobOutParameters = null;
						JobErrorCode = Remote.Interface.EndJobSpecification( Remote.Handle, EndJobInParameters, ref EndJobOutParameters );
						if( JobErrorCode >= 0 )
						{
							Job.Manager.Log( EVerbosityLevel.Informative, ELogColour.Green, "[EndJobSpecification] Started job on remote connection" );
						}
					}
				}
				if( JobErrorCode < 0 )
				{
					Job.Manager.Log( EVerbosityLevel.Informative, ELogColour.Orange, "[EndJobSpecification] Tried to begin the job on a valid remote connection, but failed" );
					Hashtable CloseOwnerInParameters = null;
					Hashtable CloseOwnerOutParameters = null;
					Job.Manager.CloseConnection( Remote.Handle, CloseOwnerInParameters, ref CloseOwnerOutParameters );
				}
			}
		}

		static void OpenJobOnRemotesThreadProc( Object ObjectState )
		{
			AgentJob Job = ObjectState as AgentJob;
			Debug.Assert( Job != null );
			Debug.Assert( Job.OwnerIsInstigator );

			LocalConnection LocalOwner = Job.Owner as LocalConnection;

			// Check if the job is in determinsitic distribution mode
			if( Job.DeterministicModeEnabled )
			{
				// For each child, simply spawn the open job thread for it
				foreach( RemoteConnection Remote in Job.Owner.RemoteChildren.Values )
				{
					// For each remote connection, spawn a thread to start the job
					ThreadPool.QueueUserWorkItem(
						new WaitCallback( OpenJobOnRemoteThreadProc ),
						new OpenJobOnRemoteData( Job, Remote )
					);
				}
			}
			// Otherwise, we're in standard distribuion mode, so only grab help as needed
			else
			{
				bool KeepLookingForHelp = ( ( Job.CurrentState == JobState.AGENT_JOB_RUNNING ) &&
											( Job.PendingTasks.Count > AgentApplication.DeveloperOptions.LocalJobsDefaultProcessorCount ) &&
											( Job.Manager.ResetPotentialRemoteAgents( LocalOwner ) ) );
				while( KeepLookingForHelp )
				{
					// Get another agent and request help. Note that just getting a remote
					// agent back here assures that we've opened and validated a connection
					// with it and it should be ready to work for us
					RemoteConnection Remote = Job.Manager.GetNextRemoteAgent( LocalOwner );
					if( Remote != null )
					{
						// For each remote connection, spawn a thread to start the job
						ThreadPool.QueueUserWorkItem(
							new WaitCallback( OpenJobOnRemoteThreadProc ),
							new OpenJobOnRemoteData( Job, Remote )
						);
					}
					else
					{
						// Subsequent passes will trickle through the unavailable agents until
						// it exhausts the set. Try to requeue any unavailable remote agents
						// (returns true if there are any)
						Job.Manager.RetryUnavailableRemoteAgent( LocalOwner );
						Thread.Sleep( 1000 );
					}

					// Update our "keep running" condition
					Int32 RunningThreads = AgentApplication.DeveloperOptions.LocalJobsDefaultProcessorCount * ( 1 + Job.Owner.RemoteChildren.Count );
					KeepLookingForHelp = ( ( Job.CurrentState == JobState.AGENT_JOB_RUNNING ) &&
										   ( Job.PendingTasks.Count > RunningThreads ) );
				}

				// Done looking for help, make sure we clean up pending connections
				Job.Manager.AbandonPotentialRemoteAgents( LocalOwner );
			}
		}

		private bool ValidateDeterministicDistributionRequirements()
		{
			// If there's no record of the last run, then fail immediately
			if( Manager.LastSuccessfulJobRecord == null )
			{
				return false;
			}

			AgentJobRecord LastSuccessfulJobRecord = Manager.LastSuccessfulJobRecord;
			bool DeterministicModeValidated = false;

			// First, compare the job specification
			bool ExecutableNameMatches = false;
			string ThisOriginalExecutableName;
			if( Specification.DependenciesOriginalNames.TryGetValue( Specification.ExecutableName, out ThisOriginalExecutableName ) )
			{
				string LastOriginalExecutableName;
				if( LastSuccessfulJobRecord.Specification.DependenciesOriginalNames.TryGetValue( LastSuccessfulJobRecord.Specification.ExecutableName, out LastOriginalExecutableName ) )
				{
					if( ThisOriginalExecutableName == LastOriginalExecutableName )
					{
						ExecutableNameMatches = true;
					}
					else
					{
						Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[Deterministic] Failed to validate Job, executable name is different" );
					}
				}
			}
			// If the executable name matches, check the set of tasks
			if( ExecutableNameMatches )
			{
				bool AllTasksMatch = false;

				// Next, make sure that all the tasks in the current specification are in the last job
				List<AgentTask> ListOfTasks = new List<AgentTask>( PendingTasks.ToArray() );
				if( ListOfTasks.Count == LastSuccessfulJobRecord.AllTasks.Count )
				{
					AgentTask NextTask;
					Int32 MatchedTaskCount = 0;
					foreach( AgentTask Task in ListOfTasks )
					{
						if( LastSuccessfulJobRecord.AllTasks.TryGetValue( Task.Specification.TaskGuid, out NextTask ) )
						{
							// For now, just compare the parameters and the cost
							if( ( NextTask.Specification.Parameters == Task.Specification.Parameters ) &&
								( NextTask.Specification.Cost == Task.Specification.Cost ) )
							{
								MatchedTaskCount++;
							}
							else
							{
								string OldTaskDetails = String.Format( "[Deterministic]     Old Task: {0}, {1}, {2}",
									NextTask.Specification.TaskGuid,
									NextTask.Specification.Parameters,
									NextTask.Specification.Cost );
								string NewTaskDetails = String.Format( "[Deterministic]     New Task: {0}, {1}, {2}",
									Task.Specification.TaskGuid,
									Task.Specification.Parameters,
									Task.Specification.Cost );
								Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[Deterministic] Failed to validate Job, task is different" );
								Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, OldTaskDetails );
								Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, NewTaskDetails );
							}
						}
					}
					if( MatchedTaskCount == LastSuccessfulJobRecord.AllTasks.Count )
					{
						// Found all current tasks in the last job record
						AllTasksMatch = true;

						// Now verify and initialize the running work queues
						LastSuccessfulJobRecord.AgentToTaskQueueMapping.Clear();
						foreach( string WorkerName in LastSuccessfulJobRecord.WorkerAgentNames )
						{
							Queue<AgentTask> TaskQueue;
							if( LastSuccessfulJobRecord.AgentToGoldenTaskQueueMapping.TryGetValue( WorkerName, out TaskQueue ) )
							{
								LastSuccessfulJobRecord.AgentToTaskQueueMapping.Add( WorkerName, new Queue<AgentTask>( TaskQueue ) );
							}
							else
							{
								// If we fail to set the enumerators, fail to validate
								AllTasksMatch = false;
							}
						}
					}
				}
				else
				{
					Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[Deterministic] Failed to validate Job, overall task count is different" );
				}
				// If all tasks are the same (enough), make sure we can acquire the same set of agents for the job
				if( AllTasksMatch )
				{
					// If all tasks match, try to connection to each of the
					// remote agents from the previous job
					Int32 WorkerAgentsEmployed = 0;
					for( Int32 i = 0; i < LastSuccessfulJobRecord.WorkerAgentNames.Count; i++ )
					{
						string WorkerAgentName = LastSuccessfulJobRecord.WorkerAgentNames[i];
						string WorkerAgentIPAddress = LastSuccessfulJobRecord.WorkerAgentIPAddresses[i];

						// Don't try to connect to self and only if this name is allowed
						if( ( WorkerAgentName != System.Net.Dns.GetHostName()) &&
							( Manager.AgentNamePassesAllowedAgentsFilter( WorkerAgentName ) ) )
						{
							// Make sure we can open the connection and get a valid remote connection back
							RemoteConnection Remote;
							AgentInfo NewWorkerInfo = new AgentInfo();
							NewWorkerInfo.Name = WorkerAgentName;
							NewWorkerInfo.Configuration["IPAddress"] = WorkerAgentIPAddress;
							if( ( Manager.TryOpenRemoteConnection( Owner as LocalConnection, NewWorkerInfo, false, DateTime.MinValue, out Remote ) == Constants.SUCCESS ) &&
								( Remote != null ) )
							{
								WorkerAgentsEmployed++;
							}
							else
							{
								Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[Deterministic] Could not acquire connection to " + WorkerAgentName + " necessary for job" );
							}
						}
						else
						{
							// We always count ourselves
							WorkerAgentsEmployed++;
						}
					}
					if( WorkerAgentsEmployed == LastSuccessfulJobRecord.WorkerAgentNames.Count )
					{
						// If all necessary agents are employed, we've verified everything
						// we can -> success
						Manager.Log( EVerbosityLevel.Informative, ELogColour.Green, "[Deterministic] Job validated, deterministic distribution enabled" );
						DeterministicModeValidated = true;
					}
					else
					{
						Manager.Log( EVerbosityLevel.Critical, ELogColour.Red, "[Deterministic] Failed to validate Job, could not acquire all agents necessary" );
					}
				}
			}
			return DeterministicModeValidated;
		}

		public Int32 EndJobSpecification()
		{
			Int32 ErrorCode = Constants.INVALID;

			// Only if the job is in the pending state can we start it running
			if( CurrentState == JobState.AGENT_JOB_PENDING )
			{
				// Sort the incoming task list by cost
				lock( PendingTasks )
				{
					List<AgentTask> ListOfTasksDescending = new List<AgentTask>( PendingTasks.ToArray() );
                    ListOfTasksDescending.Sort(AgentTask.CompareTasksByCostDescending);

                    List<AgentTask> StripedListOfTasks = ListOfTasksDescending;

                    // Lightmass processing of a single task is multithreaded (see ProcessCacheIndirectLightingTask), so ideally we would hand out the most expensive tasks in a round robin ordering
                    bool bDistributeMostExpensiveTasksRoundRobin = true;
                    int TasksPerAgent = Math.Max(AgentApplication.DeveloperOptions.LocalJobsDefaultProcessorCount, AgentApplication.DeveloperOptions.RemoteJobsDefaultProcessorCount);
                    int NumStripes = (ListOfTasksDescending.Count + TasksPerAgent - 1) / TasksPerAgent;

                    if (bDistributeMostExpensiveTasksRoundRobin && TasksPerAgent < ListOfTasksDescending.Count)
                    {
                        StripedListOfTasks = new List<AgentTask>(ListOfTasksDescending.Count);
                        int SourceIndex = 0;

                        List<bool> SourceTaskCopiedArray = new List<bool>(ListOfTasksDescending.Count);

                        for (int TaskIndex = 0; TaskIndex < ListOfTasksDescending.Count; TaskIndex++)
                        {
                            SourceTaskCopiedArray.Add(false);
                        }

                        for (int TaskIndex = 0; TaskIndex < ListOfTasksDescending.Count; TaskIndex++)
                        {
                            SourceTaskCopiedArray[SourceIndex] = true;
                            StripedListOfTasks.Add(ListOfTasksDescending[SourceIndex]);

                            SourceIndex += NumStripes;

                            if (SourceIndex >= ListOfTasksDescending.Count)
                            {
                                int NumWraparounds = (int)((ulong)(TaskIndex + 1) * (ulong)NumStripes / (ulong)ListOfTasksDescending.Count);
                                Debug.Assert(NumWraparounds >= 0);

                                SourceIndex = NumWraparounds;
                            }
                        }

                        for (int TaskIndex = 0; TaskIndex < ListOfTasksDescending.Count; TaskIndex++)
                        {
                            Debug.Assert(SourceTaskCopiedArray[TaskIndex]);
                        }
                    }

                    // Queue the sorted list back up
                    PendingTasks.Clear();
                    for (int TaskIndex = StripedListOfTasks.Count - 1; TaskIndex >= 0; TaskIndex--)
                    {
                        // Reverse the order as PendingTasks is a stack, the most expensive tasks should be on the top
                        PendingTasks.Push(StripedListOfTasks[TaskIndex]);
                    }
				}
	
				// We're pessimists
				bool AllowJobToStart = false;

				// Determine if we meet conditions for distribution
				bool DistributionConditionsMet = ( OwnerIsInstigator ) &&
												 ( AgentApplication.Options.EnableStandaloneMode == false ) &&
												 ( ( Specification.JobFlags & EJobTaskFlags.FLAG_MANUAL_START ) == 0 ) &&
												 ( ( Specification.JobFlags & EJobTaskFlags.FLAG_ALLOW_REMOTE ) != 0 );

				// Check if the user enabled deterministic distribution mode
				if( ( DistributionConditionsMet ) &&
					( AgentApplication.DeveloperOptions.ReplayLastDistribution ) )
				{
					if( ValidateDeterministicDistributionRequirements() )
					{
						// If enabled and verified, set the deterministic mode flag
						// and allow the job to start normally
						DeterministicModeEnabled = true;
						AllowJobToStart = true;
					}
				}
				else
				{
					// Otherwise, allow the job to start normally
					AllowJobToStart = true;
				}

				// If we should still start the job normally after the conditions above
				if( AllowJobToStart )
				{
					ErrorCode = StartJobExecution();
					if( ErrorCode >= 0 )
					{
						// Update the visualizer if the owner is the Instigator
						if( OwnerIsInstigator )
						{
							// Send the total task count to the visualizer if this agent is the Instigator
							AgentApplication.UpdateMachineState(System.Net.Dns.GetHostName(), ( Int32 )TaskCount, EProgressionState.TaskTotal );
						}

						// If we have more than N pending tasks, the job owner is the instigator,
						// and the Agent settings and the job specification allow distribution,
						// request the help of other, remote agents (N is currently the number of
						// allowed worker threads for local job executables)
						if(	( DistributionConditionsMet ) &&
							( PendingTasks.Count > AgentApplication.DeveloperOptions.LocalJobsDefaultProcessorCount ) )
						{
							// Spawn a thread to manage adding more agents to the job
							ThreadPool.QueueUserWorkItem( new WaitCallback( OpenJobOnRemotesThreadProc ), this );
						}
					}
				}
			}

			return ( ErrorCode );
		}

		public Int32 CloseJob()
		{
			// Before we close the job, make sure all messages have been processed to
			// make sure we avoid any race condition between getting updates to tasks
			// or the job and closing the job
			Manager.FlushMessageQueue( Owner, false );

			// Update the state only within a mutex to protect anyone trying to read
			// the state at the same time
			lock( CurrentStateLock )
			{
				// Only do this if this Job hasn't already been closed
				if( CurrentState != JobState.AGENT_JOB_CLOSED )
				{
					CurrentState = JobState.AGENT_JOB_CLOSED;
                    StopTime = DateTime.UtcNow;

					// First, resolve any outstanding reservations
					CheckForReservations();

					// Determine success state of the Job
					lock( CurrentSuccessStateLock )
					{
						// Only update if the Job success state has not been determined already by other means
						if( CurrentSuccessState == JobSuccessState.AGENT_JOB_INCOMPLETE )
						{
							// If this is an agent managed, Task-based process
							if( ( Specification != null ) &&
								( ( Specification.JobFlags & EJobTaskFlags.FLAG_MANUAL_START ) == 0 ) &&
								( TaskCount > 0 ) )
							{
								// Check for a set of known failure cases
								bool IsAStandardFailureCase = false;
								string NewMessageText = "No message provided!";

								// If there are any Tasks still pending
								if( PendingTasks.Count != 0 )
								{
									// Log and send an INFO message describing the failure
									NewMessageText = "Job has failed! Job is closed while Tasks are still PENDING";
									IsAStandardFailureCase = true;
								}
								// If there are any Tasks still running
								else if( RunningTasks.Count != 0 )
								{
									// Log and send an INFO message describing the failure
									NewMessageText = "Job has failed! Job is closed while Tasks are still RUNNING";
									IsAStandardFailureCase = true;
								}
								// If any Task was reported a failure
								else if( TaskFailureCount != 0 )
								{
									// Log and send an INFO message describing the failure
									NewMessageText = "Job has failed! The task failure count is non-zero";
									IsAStandardFailureCase = true;
								}

								if( IsAStandardFailureCase )
								{
									CurrentSuccessState = JobSuccessState.AGENT_JOB_FAILURE;
									if( OwnerIsInstigator )
									{
										SendJobCompletedMessage( new AgentInfoMessage( NewMessageText ) );
									}
								}
								else
								{
									// This is the only way to mark an agent managed, Task-based Job a success
									CurrentSuccessState = JobSuccessState.AGENT_JOB_SUCCESS;
									if( OwnerIsInstigator )
									{
										// Log and send an INFO message describing the failure
										NewMessageText = "Job is a success!";
										SendJobCompletedMessage( new AgentInfoMessage( NewMessageText ) );
									}
								}
							}
							else
							{
								// Otherwise, the process should shut itself down now that the Job
								// has been ended and any reservations have been sent out. If it
								// fails to quit itself, it will be killed and we'll still get
								// the exited process callback
							}
						}
					}

					// For each remote connection we have for this Job, end the Job.
					// Ending the Job will eventually cause remote Job executables to
					// be notified that the Job is closed.
					foreach( RemoteConnection RemoteChild in Owner.RemoteChildren.Values )
					{
						Hashtable CloseJobInParameters = null;
						Hashtable CloseJobOutParameters = null;
						RemoteChild.Interface.CloseJob( RemoteChild.Handle, CloseJobInParameters, ref CloseJobOutParameters );
					}

					// If this is the Instigator, perform additional post-job work
					if( OwnerIsInstigator )
					{
						// Inform the visualizer that we've disconnected
						AgentApplication.UpdateMachineState(System.Net.Dns.GetHostName(), -1, EProgressionState.InstigatorDisconnected );

						// If the job was a success, record the state for determinisitc replay
						if( CurrentSuccessState == JobSuccessState.AGENT_JOB_SUCCESS )
						{
							// We're done with the last run record now
							Manager.LastSuccessfulJobRecord = null;

							bool DeterministicModeAllowed = true;
							AgentJobRecord NewJobRecord = new AgentJobRecord();
							NewJobRecord.Specification = Specification;
							
							// Sort all retired tasks by assign time to make sure the order is correct
							List<AgentTask> ListOfRetiredTasks = new List<AgentTask>( RetiredTasks.ToArray() );
							ListOfRetiredTasks.Sort( AgentTask.CompareTasksByAssignTime );

							// Assign out the tasks based on where it was assigned and when
							foreach( AgentTask NextTask in ListOfRetiredTasks )
							{
								// Add the task to the set of all tasks
								NewJobRecord.AllTasks.Add( NextTask.Specification.TaskGuid, NextTask );

								// Add the task to the agent-specific queue, creating an entry
								// for the agent if we haven't seen it yet
								Queue<AgentTask> TaskQueue = null;
								string NameOfWorker = Manager.MachineNameFromConnection( NextTask.CurrentOwner );
								string IPAddressOfWorker = Manager.MachineIPAddressFromConnection( NextTask.CurrentOwner );
								if( !NewJobRecord.WorkerAgentNames.Contains( NameOfWorker ) )
								{
									// Create a new task queue for the newly discovered agent
									TaskQueue = new Queue<AgentTask>();
									TaskQueue.Enqueue( NextTask );

									// Add this new agent to both the set of names and the task mapping sets
									NewJobRecord.WorkerAgentNames.Add( NameOfWorker );
									NewJobRecord.WorkerAgentIPAddresses.Add( IPAddressOfWorker );
									NewJobRecord.AgentToGoldenTaskQueueMapping.Add( NameOfWorker, TaskQueue );
								}
								else if( NewJobRecord.AgentToGoldenTaskQueueMapping.TryGetValue( NameOfWorker, out TaskQueue ) )
								{
									// Queue up the next task
									TaskQueue.Enqueue( NextTask );
								}
								else
								{
									// Error, we should fail the entire thing
									DeterministicModeAllowed = false;
									break;
								}
							}

							// If allowed, assign it for the next run
							if( DeterministicModeAllowed )
							{
								Manager.LastSuccessfulJobRecord = NewJobRecord;
							}
						}

						// Report some of the stats for the Job
						// For each successfully completed task, log the time it took and the time/cost
						foreach( AgentTask NextTask in RetiredTasks.ToArray() )
						{
							if( NextTask.CurrentState.TaskState == EJobTaskState.TASK_STATE_COMPLETE_SUCCESS )
							{
								TimeSpan ScheduledTime = NextTask.StartTime - NextTask.AssignTime;
								TimeSpan RunningTime = NextTask.StopTime - NextTask.StartTime;
								string LogMessage = String.Format( "[CloseJob] Task {0} {1} - Scheduled(ms): {2}, Running(ms): {3}, Cost: {4}, Running(ms)/Cost: {5}",
									NextTask.Specification.TaskGuid,
									NextTask.Specification.Parameters,
									ScheduledTime.TotalMilliseconds,
									RunningTime.TotalMilliseconds,
									NextTask.Specification.Cost,
									( double )RunningTime.TotalMilliseconds / ( double )NextTask.Specification.Cost );
								Manager.Log( EVerbosityLevel.Verbose, ELogColour.Green, LogMessage );
							}
						}
					}
				}
			}

			// Attempt to report the final stats to the DB
			PostJobStatsToDB();

			return ( Constants.SUCCESS );
		}

		private bool AvoidNextTask( Connection RequestingConection )
		{
			Debug.Assert( RequestingConection.Job != null );
			bool SafeToAvoidNextTask =
				( AgentApplication.Options.AvoidLocalExecution ) &&
				( RequestingConection is LocalConnection ) &&
				( RequestingConection.Job.Owner.RemoteChildren.Count > 0 );

			return SafeToAvoidNextTask;
		}

		public AgentTaskRequestResponse GetNextTask( Connection RequestingConnection )
		{
			// We need the state to not change while we're in here
			lock( CurrentStateLock )
			{
				// Take the next Task off of the Pending list and add it to the Active list
				if( CurrentState == JobState.AGENT_JOB_RUNNING )
				{
					lock( PendingTasks )
					{
						AgentTask NextTask = null;

						// Check for whether we're running in deterministic mode
						if( DeterministicModeEnabled )
						{
							// If so, look up the requesting connection's task queue
							Queue<AgentTask> TaskQueue;
							string RequestorName = Manager.MachineNameFromConnection( RequestingConnection );
							if( Manager.LastSuccessfulJobRecord.AgentToTaskQueueMapping.TryGetValue( RequestorName, out TaskQueue ) )
							{
								if( TaskQueue.Count > 0 )
								{
									// Grab the next pending task
									NextTask = TaskQueue.Dequeue();
								}
							}
						}
						// Otherwise, we're in a standard distribution mode, see if there
						// are tasks available to hand out
						else if( PendingTasks.Count > 0 )
						{
							// Check to see if should avoid the next task
							if( !AvoidNextTask( RequestingConnection ) )
							{
								NextTask = PendingTasks.Pop();
							}
						}

						if( NextTask != null )
						{
							// Assign the new owner and add the task to the running sets
							NextTask.CurrentOwner = RequestingConnection;
							
							// Set the assign time of the task as well as the start time,
							// in case they don't send back a RUNNING message, which will
							// set the proper start time
							NextTask.AssignTime = DateTime.UtcNow;
							NextTask.StartTime = DateTime.UtcNow;

							// Update additional stats
							if( RequestingConnection is RemoteConnection )
							{
								TaskCountRemote++;
							}

							AgentTaskSpecification TaskSpecification = NextTask.Specification;
							RequestingConnection.RunningTasks.Add( TaskSpecification.TaskGuid, NextTask );
							RunningTasks.Add( TaskSpecification.TaskGuid, NextTask );

							// Send back the now ready task specification
							return TaskSpecification;
						}
						// If we're not the final authority of this job, then we'll send back a
						// reservation and allow the Instigator send the real answer later.
						// Alternately, if the owner is the Instigator, we also need to send
						// back a reservation to make sure the local job application runs until
						// the job is done.
						else if( ( OwnerIsInstigator == false ) ||
								 ( RequestingConnection is LocalConnection ) )
						{
							// Always track outstanding reservations
							RequestingConnection.ReservationCount++;
							TaskReservationCount++;

							// For local connections, send a RESERVATION response which indicates that we
							// have no work currently to hand out, but we might in the future, so sit
							// tight. For remote connections, we only need to track the reservation
							if( RequestingConnection is LocalConnection )
							{
								return ( new AgentTaskRequestResponse( JobGuid,
																	   ETaskRequestResponseType.RESERVATION ) );
							}
						}
						// Otherwise, we're all out of tasks, so release the worker
						else
						{
							// No more distributed tasks to hand out, send back a release
							return ( new AgentTaskRequestResponse( JobGuid,
																   ETaskRequestResponseType.RELEASE ) );
						}
					}
				}
				else if( CurrentState == JobState.AGENT_JOB_CLOSED )
				{
					// No more distributed tasks to hand out, send back a release
					return ( new AgentTaskRequestResponse( JobGuid,
														   ETaskRequestResponseType.RELEASE ) );
				}
			}
			return null;
		}

		public void CancelReservations( Connection ReservationHolder )
		{
			// We need the state to not change while we're in here
			lock( CurrentStateLock )
			{
				TaskReservationCount -= ReservationHolder.ReservationCount;
				ReservationHolder.ReservationCount = 0;
			}
		}

		public void CheckForReservations()
		{
			// We need the state to not change while we're in here
			lock( CurrentStateLock )
			{
				// Depending on the current Job state, handle any outstanding Task reservations
				if( ( CurrentState == JobState.AGENT_JOB_UNSPECIFIED ) ||
					( CurrentState == JobState.AGENT_JOB_PENDING ) )
				{
					// As long as the job is PENDING, do nothing. In fact, there shouldn't be
					// any reservations at this point. Assert this.
					Debug.Assert( TaskReservationCount == 0 );
				}
				else if( CurrentState == JobState.AGENT_JOB_RUNNING )
				{
					// If there are any outstanding reservations and tasks to give out...
					UInt32 PendingTasksCount = ( UInt32 )PendingTasks.Count;
					UInt32 ReservationsToTryToFill = Math.Min( PendingTasksCount, TaskReservationCount );

					if( ReservationsToTryToFill > 0 )
					{
						// Try to fulfill the local ones first
						List<LocalConnection> LocalChildren = Owner.LocalChildren.Values;
						List<LocalConnection>.Enumerator Children = LocalChildren.GetEnumerator();

						while( ( Children.MoveNext() ) &&
							   ( ReservationsToTryToFill > 0 ) )
						{
							LocalConnection Local = Children.Current;
							while( ( Local.ReservationCount > 0 ) &&
								   ( ReservationsToTryToFill > 0 ) )
							{
								// If we should avoid assigning tasks to this agent,
								// simply continue on to the next one (only meaningful
								// for local connections since we currently would only
								// avoid tasks on the Instigator)
								if( AvoidNextTask( Local ) )
								{
									break;
								}

								AgentTaskRequestResponse Response = GetNextTask( Local );
								if( ( Response != null ) &&
									( Response is AgentTaskSpecification ) )
								{
									AgentTaskSpecification NextTask = Response as AgentTaskSpecification;

									// For local connections, send the task message directly
									Response.To = Local.Handle;
									Response.From = Owner.Handle;
									Manager.SendMessageInternal( Owner, Response );

									// Consider the reservation satisfied
									TaskReservationCount--;
									Local.ReservationCount--;
									ReservationsToTryToFill--;
								}
								else
								{
									// If we didn't get a task, don't keep trying for now
									ReservationsToTryToFill = 0;
								}
							}
						}
					}

					// If we still have pending tasks try to fulfill any outstanding remote reservations
					if( ReservationsToTryToFill > 0 )
					{
						List<RemoteConnection> RemoteChildren = Owner.RemoteChildren.Values;
						List<RemoteConnection>.Enumerator Children = RemoteChildren.GetEnumerator();

						while( ( Children.MoveNext() ) &&
							   ( ReservationsToTryToFill > 0 ) )
						{
							RemoteConnection Remote = Children.Current;
							while( ( Remote.ReservationCount > 0 ) &&
								   ( ReservationsToTryToFill > 0 ) )
							{
								AgentTaskRequestResponse Response = GetNextTask( Remote );
								if( ( Response != null ) &&
									( Response is AgentTaskSpecification ) )
								{
									AgentTaskSpecification NextTask = Response as AgentTaskSpecification;

									// For remote connections, add the task using the job/task API
									Hashtable AddTaskInParameters = new Hashtable();
									AddTaskInParameters["Version"] = ESwarmVersionValue.VER_1_0;
									AddTaskInParameters["Specification"] = NextTask;
									Hashtable AddTaskOutParameters = null;

									if( Remote.Interface.AddTask( Remote.Handle, AddTaskInParameters, ref AddTaskOutParameters ) >= 0 )
									{
										// Consider the reservation satisfied
										TaskReservationCount--;
										Remote.ReservationCount--;
										ReservationsToTryToFill--;
									}
									else
									{
										// If this fails for any reason, update the task and move on to the next child.
										// Note that marking it KILLED will allow it to be requeued for another agent
										// and ultimately it'll end up on the local agent, where if it fails, it'll
										// be marked a real failure.
										UpdateTaskState( new AgentTaskState( NextTask.JobGuid, NextTask.TaskGuid, EJobTaskState.TASK_STATE_KILLED ) );
										break;
									}
								}
								else
								{
									// If we didn't get a task, don't keep trying for now
									ReservationsToTryToFill = 0;
								}
							}
						}
					}
				}
				else if( CurrentState == JobState.AGENT_JOB_CLOSED )
				{
					// If there are any outstanding reservations, we can release them now that we know
					// there will be no more tasks added to the Job
					if( TaskReservationCount > 0 )
					{
						foreach( LocalConnection Local in Owner.LocalChildren.Values )
						{
							// If there are any outstanding reservations, a single RELEASE is sufficient
							if( Local.ReservationCount > 0 )
							{
								AgentTaskRequestResponse ReleaseMessage =
									new AgentTaskRequestResponse( JobGuid, ETaskRequestResponseType.RELEASE );
								ReleaseMessage.To = Local.Handle;
								ReleaseMessage.From = Owner.Handle;
								Manager.SendMessageInternal( Owner, ReleaseMessage );
							}
							// Consider all reservations canceled
							TaskReservationCount -= Local.ReservationCount;
							Local.ReservationCount = 0;
						}

						foreach( RemoteConnection Remote in Owner.RemoteChildren.Values )
						{
							// If there are any outstanding reservations, a single RELEASE is sufficient
							if( Remote.ReservationCount > 0 )
							{
								AgentTaskRequestResponse ReleaseMessage =
									new AgentTaskRequestResponse( JobGuid, ETaskRequestResponseType.RELEASE );
								ReleaseMessage.To = Remote.Handle;
								ReleaseMessage.From = Owner.Handle;
								Manager.SendMessageInternal( Owner, ReleaseMessage );
							}
							// Consider all reservations canceled
							TaskReservationCount -= Remote.ReservationCount;
							Remote.ReservationCount = 0;
						}

						// Sanity check
						Debug.Assert( TaskReservationCount == 0 );
					}
				}
			}
		}

		private void UpdateTaskStateAsSuccess( AgentTask RunningTask )
		{
			TaskSuccessCount++;
            RunningTask.StopTime = DateTime.UtcNow;
			RunningTasks.Remove( RunningTask.Specification.TaskGuid );
			RunningTask.CurrentOwner.RunningTasks.Remove( RunningTask.Specification.TaskGuid );
			RetiredTasks.Enqueue( RunningTask );
		}

		private void UpdateTaskStateAsFailure( AgentTask RunningTask )
		{
			TaskFailureCount++;
            RunningTask.StopTime = DateTime.UtcNow;
			RunningTasks.Remove( RunningTask.Specification.TaskGuid );
			RunningTask.CurrentOwner.RunningTasks.Remove( RunningTask.Specification.TaskGuid );
			RetiredTasks.Enqueue( RunningTask );
		}

		private void UpdateTaskStateAsRequeued( AgentTask RunningTask )
		{
			// A task is killed or rejected when a connection is closed and it still has
			// pending or running tasks assigned to it. Simply requeue the tasks for
			// another agent to pick up.

            RunningTask.StopTime = DateTime.UtcNow;
			RunningTasks.Remove( RunningTask.Specification.TaskGuid );
			RunningTask.CurrentOwner.RunningTasks.Remove( RunningTask.Specification.TaskGuid );

			// Update stats
			if( RunningTask.CurrentOwner is RemoteConnection )
			{
				TaskCountRemote--;
			}

			// Reset a couple states before requeueing
			RunningTask.CurrentState = new AgentTaskState( RunningTask.Specification.JobGuid, RunningTask.Specification.TaskGuid, EJobTaskState.TASK_STATE_IDLE );
			RunningTask.CurrentOwner = null;

			PendingTasks.Push( RunningTask );
			TaskCountRequeue++;

			// With the killed task requeued, check for reservations to see
			// if anyone else can take the task on right away
			CheckForReservations();
		}

		public void UpdateTaskState( AgentTaskState UpdatedTaskState )
		{
			// Sanity checks
			Debug.Assert( CurrentState != JobState.AGENT_JOB_UNSPECIFIED );
			Debug.Assert( CurrentState != JobState.AGENT_JOB_PENDING );

			AgentTask RunningTask;
			if( RunningTasks.TryGetValue( UpdatedTaskState.TaskGuid, out RunningTask ) )
			{
				// Update the individual Task state
				RunningTask.CurrentState = UpdatedTaskState;
				switch( UpdatedTaskState.TaskState )
				{
					case EJobTaskState.TASK_STATE_ACCEPTED:
						// Nothing to do right now, but we'll need to track start times, etc. later
						break;

					case EJobTaskState.TASK_STATE_RUNNING:
						// Mark the real start time of this task (also set when we give the task out)
						RunningTask.StartTime = DateTime.UtcNow;
						break;

					case EJobTaskState.TASK_STATE_COMPLETE_SUCCESS:
						UpdateTaskStateAsSuccess( RunningTask );
						break;

					case EJobTaskState.TASK_STATE_REJECTED:
						if( RunningTask.CurrentOwner is RemoteConnection )
						{
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Orange, "[UpdateTaskState]: Task Rejected remotely by " + ( RunningTask.CurrentOwner as RemoteConnection ).Info.Name );
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Orange, "[UpdateTaskState]: Requeueing: " + RunningTask.Specification.Parameters );
							UpdateTaskStateAsRequeued( RunningTask );
						}
						else
						{
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, "[UpdateTaskState]: Task Rejected locally by " + System.Net.Dns.GetHostName() + ", counted as failure");
							UpdateTaskStateAsFailure( RunningTask );
						}
						break;

					case EJobTaskState.TASK_STATE_KILLED:
						if( RunningTask.CurrentOwner is RemoteConnection )
						{
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Orange, "[UpdateTaskState]: Task Killed remotely by " + ( RunningTask.CurrentOwner as RemoteConnection ).Info.Name );
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Orange, "[UpdateTaskState]: Requeueing: " + RunningTask.Specification.Parameters );
							UpdateTaskStateAsRequeued( RunningTask );
						}
						else
						{
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, "[UpdateTaskState]: Task Killed locally by " + System.Net.Dns.GetHostName() + ", counted as failure" );
							UpdateTaskStateAsFailure( RunningTask );
						}
						break;

					case EJobTaskState.TASK_STATE_COMPLETE_FAILURE:
						if( RunningTask.CurrentOwner is RemoteConnection )
						{
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, "[UpdateTaskState]: Task Failed on " + ( RunningTask.CurrentOwner as RemoteConnection ).Info.Name );
						}
						else
						{
							Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, "[UpdateTaskState]: Task Failed on " + System.Net.Dns.GetHostName());
						}
						Manager.Log( EVerbosityLevel.Informative, ELogColour.Red, "[UpdateTaskState]: Task Failed: " + RunningTask.Specification.Parameters );
						UpdateTaskStateAsFailure( RunningTask );
						break;
				}

				// Update the owning Job state, by checking for failures. Success is only
				// determined after all Tasks are assured to be done or orphanable. Only
				// do this one time and let the new state, if there is one, be sticky.
				lock( CurrentSuccessStateLock )
				{
					if( CurrentSuccessState == JobSuccessState.AGENT_JOB_INCOMPLETE )
					{
						// Updtae if any task is a failure
						if( TaskFailureCount > 0 )
						{
							// Update the state and send a message indicating the failure
							CurrentSuccessState = JobSuccessState.AGENT_JOB_FAILURE;
							if( OwnerIsInstigator )
							{
								// Log and send an INFO message describing the failure
								string NewMessageText = "Job has failed! The task failure count is non-zero"; 
								SendJobCompletedMessage( new AgentInfoMessage( NewMessageText ) );
							}
						}
						// Update if all tasks are successful and we're the instigator, since only
						// the instigator can make this determination properly
						else if( ( TaskSuccessCount == TaskCount ) &&
								 ( OwnerIsInstigator ) )
						{
							CurrentSuccessState = JobSuccessState.AGENT_JOB_SUCCESS;
							if( OwnerIsInstigator )
							{
								// Log and send an INFO message describing the success
								string NewMessageText = "Job is a success!";
								SendJobCompletedMessage( new AgentInfoMessage( NewMessageText ) );
							}
						}
					}
				}

				// Update the visualizer if this agent is the Instigator
				if( OwnerIsInstigator )
				{
					AgentApplication.UpdateMachineState(System.Net.Dns.GetHostName(), RetiredTasks.Count, EProgressionState.TasksCompleted );
					AgentApplication.UpdateMachineState(System.Net.Dns.GetHostName(), RunningTasks.Count, EProgressionState.TasksInProgress );
				}
			}
		}

		// Specification for the Job
		public Agent Manager = null;
		public Connection Owner = null;
		public bool OwnerIsInstigator = false;
		public bool DeterministicModeEnabled = false;

		public AgentGuid JobGuid;
		public AgentJobSpecification Specification32 = null;
		public AgentJobSpecification Specification64 = null;
		public AgentJobSpecification Specification = null;

		// Specifications for Tasks belonging to the Job
		public UInt32 TaskCount = 0;
		public UInt32 TaskCountRemote = 0;
		public UInt32 TaskCountRequeue = 0;
		public UInt32 TaskSuccessCount = 0;
		public UInt32 TaskFailureCount = 0;
		public UInt32 TaskReservationCount = 0;
		public ReaderWriterStack<AgentTask> PendingTasks = new ReaderWriterStack<AgentTask>();
		public ReaderWriterDictionary<AgentGuid, AgentTask> RunningTasks = new ReaderWriterDictionary<AgentGuid, AgentTask>();
		public ReaderWriterQueue<AgentTask> RetiredTasks = new ReaderWriterQueue<AgentTask>();

		// Variables used for Job process monitoring
		public JobState	CurrentState = JobState.AGENT_JOB_UNSPECIFIED;
		public Object	CurrentStateLock = new Object();
		public JobSuccessState	CurrentSuccessState = JobSuccessState.AGENT_JOB_INCOMPLETE;
		public Object			CurrentSuccessStateLock = new Object();

		public Process	ProcessObject = null;
		public Object	ProcessObjectLock = new Object();
		public Int32	ProcessObjectOutputLines = 0;

		public Int32	ProcessObjectExitCode = 0;
		public Int64	ProcessObjectPeakVirtualMemorySize64 = 0;
		public Int64	ProcessObjectPeakWorkingSet64 = 0;

		// Cache usage statistics for the Job
		public Int32	CacheRequests = 0;
		public Int32	CacheMisses = 0;
		public Int32	CacheHits = 0;

		// Simple push/pull channel statistics for the Job
		public Int64	NetworkBytesSent = 0;
		public Int64	NetworkBytesReceived = 0;

		// Optional, common description values for the job
		public Hashtable	Description32 = null;
		public Hashtable	Description64 = null;
		public Hashtable	Description = null;

		public bool		JobStatsPostedToDB = false;

		public DateTime StartTime;
		public DateTime StopTime;
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * A simple record used to keep track of the last job, including the remote
	 * agents that were employed, and where each task was distributed to
	 */
	public class AgentJobRecord
	{
		public AgentJobRecord()
		{
			Specification = null;
			WorkerAgentNames = new List<string>();
			WorkerAgentIPAddresses = new List<string>();
			AllTasks = new ReaderWriterDictionary<AgentGuid, AgentTask>();
			AgentToGoldenTaskQueueMapping = new ReaderWriterDictionary<string, Queue<AgentTask>>();
			AgentToTaskQueueMapping = new ReaderWriterDictionary<string, Queue<AgentTask>>();
		}

		public AgentJobSpecification Specification;
		public List<string> WorkerAgentNames;
		public List<string> WorkerAgentIPAddresses;
		public ReaderWriterDictionary<AgentGuid, AgentTask> AllTasks;
		public ReaderWriterDictionary<string, Queue<AgentTask>> AgentToGoldenTaskQueueMapping;
		public ReaderWriterDictionary<string, Queue<AgentTask>> AgentToTaskQueueMapping;
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of job management behavior in the Agent
	 */
	public partial class Agent : MarshalByRefObject, IAgentInternalInterface, IAgentInterface
	{
		///////////////////////////////////////////////////////////////////////////

		// The set of all active Jobs for this Agent
		public ReaderWriterDictionary<AgentGuid, AgentJob> ActiveJobs = new ReaderWriterDictionary<AgentGuid, AgentJob>();
		public AgentJobRecord LastSuccessfulJobRecord = null;

		// A special debug Job GUID
		public AgentGuid DebuggingJobGuid = new AgentGuid( 0x00000123, 0x00004567, 0x000089ab, 0x0000cdef );

		///////////////////////////////////////////////////////////////////////////

		private Int32 OpenJob_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "OpenJob_1_0-Internal", true );

            Int32 ErrorCode = Constants.INVALID;

			Connection JobOwner;
			if( Connections.TryGetValue( ConnectionHandle, out JobOwner ) )
			{
				Debug.Assert( ( JobOwner.Job == null ) ||
							  ( JobOwner.Job.CurrentState == AgentJob.JobState.AGENT_JOB_CLOSED ) );

				// Unpack the input parameters
				AgentGuid JobGuid = InParameters["JobGuid"] as AgentGuid;

				// Check for any other Jobs running
				if( ActiveJobs.Count > 0 )
				{
					foreach( AgentJob Job in ActiveJobs.Values )
					{
						// If any remote jobs are running, we'll kill them
						if( Job.Owner is RemoteConnection )
						{
							// We need to close this connection, which should quit the job and release all associated resources
							Log( EVerbosityLevel.Verbose, ELogColour.Green, "[OpenJob] Interrupting remote job in favor of local one" );
							Hashtable OwnerInParameters = null;
							Hashtable OwnerOutParameters = null;
							CloseConnection( Job.Owner.Handle, OwnerInParameters, ref OwnerOutParameters );
						}
						else
						{
							Debug.Assert( Job.Owner is LocalConnection );
							
							// TODO: By allowing multiple local jobs to run, we expose
							// limitations in the visualizer that cause it to do some odd
							// things like assuming that all messages are referring to the
							// same job (and thus easily confused)
						}
					}
				}

				// Ensure that the folders we need are there
				string JobsFolder = Path.Combine( AgentApplication.Options.CacheFolder, "Jobs" );
				ErrorCode = EnsureFolderExists( JobsFolder );
				if( ErrorCode >= 0 )
				{
					// For the Job-specific folder, if it's already there, clean it out;
					// we need to be sure to start every Job with an empty Job folder
					string JobSpecificFolder = Path.Combine( JobsFolder, "Job-" + JobGuid.ToString() );
					ErrorCode = DirectoryRecreate( JobSpecificFolder );
					if( ErrorCode >= 0 )
					{
						// Create the new Job object and initialize it via the OpenJob method
						AgentJob NewJob = new AgentJob( this );
						NewJob.OpenJob( JobOwner, JobGuid );

						Log( EVerbosityLevel.Informative, ELogColour.Green, "[Job] Accepted Job " + JobGuid.ToString() );
						ErrorCode = Constants.SUCCESS;
					}
				}

				// Check for errors from above
				if( ErrorCode < 0 )
				{
					Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] OpenJob: Rejected, unable to create necessary directories" );
					ErrorCode = Constants.ERROR_JOB;
				}
			}
			else
			{
				Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] OpenJob: Rejected, unrecognized connection" );
                ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
            }

            StopTiming();
            return ErrorCode;
		}

		private Int32 BeginJobSpecification_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "BeginJobSpecification_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			Connection JobOwner;
			if( ( Connections.TryGetValue( ConnectionHandle, out JobOwner ) &&
				( JobOwner.Job != null ) ) )
			{
				// Unpack the input parameters
				AgentJobSpecification Specification32 = InParameters["Specification32"] as AgentJobSpecification;
				AgentJobSpecification Specification64 = InParameters["Specification64"] as AgentJobSpecification;

				Hashtable Description32 = null;
				if( InParameters.Contains( "Description32" ) )
				{
					Description32 = InParameters["Description32"] as Hashtable;
				}
				Hashtable Description64 = null;
				if( InParameters.Contains( "Description64" ) )
				{
					Description64 = InParameters["Description64"] as Hashtable;
				}

				ErrorCode = JobOwner.Job.BeginJobSpecification( Specification32, Description32, Specification64, Description64 );
			}
			else
			{
				Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] BeginJobSpecification: Rejected, unrecognized connection or job" );
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ErrorCode;
		}

		private Int32 AddTask_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "AddTask_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			Connection JobOwner;
			if( ( Connections.TryGetValue( ConnectionHandle, out JobOwner ) &&
				( JobOwner.Job != null ) ) )
			{
				// This is only allowed while in the PENDING state, unless the one adding the
				// Task is a remote Agent (in contrast to a local connection, i.e. an Instigator)
				if( ( JobOwner.Job.CurrentState == AgentJob.JobState.AGENT_JOB_PENDING ) ||
					( JobOwner is RemoteConnection ) )
				{
					// Unpack the input parameters and pass them along
					if( InParameters.ContainsKey( "Specification" ) )
					{
						AgentTaskSpecification Specification = InParameters["Specification"] as AgentTaskSpecification;
						ErrorCode = JobOwner.Job.AddTask( Specification );
					}
					else if( InParameters.ContainsKey( "Specifications" ) )
					{
						List<AgentTaskSpecification> Specifications = InParameters["Specifications"] as List<AgentTaskSpecification>;
						foreach( AgentTaskSpecification NextSpecification in Specifications )
						{
							ErrorCode = JobOwner.Job.AddTask( NextSpecification );
							if( ErrorCode < 0 )
							{
								// If any of the individual adds has an error, stop and return
								break;
							}
						}
					}
				}
				else
				{
					Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] AddTask: Rejected, cannot add a task to a running job" );
					ErrorCode = Constants.ERROR_JOB;
				}
			}
			else
			{
				Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] AddTask: Rejected, unrecognized connection or job" );
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ErrorCode;
		}

		private Int32 EndJobSpecification_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "EndJobSpecification_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			Connection JobOwner;
			if( ( Connections.TryGetValue( ConnectionHandle, out JobOwner ) &&
				( JobOwner.Job != null ) ) )
			{
				ErrorCode = JobOwner.Job.EndJobSpecification();
			}
			else
			{
				Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] EndJobSpecification: Rejected, unrecognized connection or job" );
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ErrorCode;
		}

		private Int32 CloseJob_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "CloseJob_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			Connection JobOwner;
			if( ( Connections.TryGetValue( ConnectionHandle, out JobOwner ) &&
				( JobOwner.Job != null ) ) )
			{
				ErrorCode = JobOwner.Job.CloseJob();
			}
			else
			{
				Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] CloseJob: Rejected, unrecognized connection or job" );
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ErrorCode;
		}

		public Int32 GetActiveJobGuid( Int32 ConnectionHandle, ref AgentGuid ActiveJobGuid )
		{
			StartTiming( "GetActiveJobGuid-Internal", true );

			ActiveJobGuid = null;
			Int32 ErrorCode = Constants.INVALID;

			Connection JobOwner;
			if( Connections.TryGetValue( ConnectionHandle, out JobOwner ) )
			{
				if( JobOwner.Job != null )
				{
					ActiveJobGuid = JobOwner.Job.JobGuid;
					ErrorCode = Constants.SUCCESS;
				}
			}
			else
			{
				Log( EVerbosityLevel.Critical, ELogColour.Red, "[Job] GetActiveJobGuid: Rejected, unrecognized connection" );
				ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ErrorCode;
		}

		public void MaintainJobs()
		{
			// If we're not initialied yet, do nothing
			if( !Initialized.WaitOne(0) )
			{
				return;
			}

			// For all running jobs, check for completion, crashes, etc
			List<AgentGuid> InactiveJobs = new List<AgentGuid>();
			foreach( AgentJob Job in ActiveJobs.Values )
			{
				// We need the state to not change while we're in here
				lock( Job.CurrentStateLock )
				{
					// We also need the process object, if it still is around, to stay around while we do this
					lock( Job.ProcessObjectLock )
					{
						// Only maintain running Jobs
						if( Job.CurrentState == AgentJob.JobState.AGENT_JOB_RUNNING )
						{
							bool IsAnAgentManagedProcess =
								( Job.Specification.JobFlags & EJobTaskFlags.FLAG_MANUAL_START ) == 0;

							// Check for fully manual, not yet attached job processes
							if( Job.ProcessObject == null )
							{
								String WaitingJobName = Job.Specification.ExecutableName + " " + Job.Specification.Parameters;
								if( IsAnAgentManagedProcess == false )
								{
									Log( EVerbosityLevel.Informative, ELogColour.Orange, "[Maintain Jobs] Job \"" + WaitingJobName + "\" is not connected yet" );
								}
								else
								{
									Log( EVerbosityLevel.ExtraVerbose, ELogColour.Orange, "[Maintain Jobs] Job \"" + WaitingJobName + "\" is either trying to start up or shut down" );
								}
							}
							else
							{
								string JobName = Job.Specification.ExecutableName;
                                TimeSpan JobRunningTime = DateTime.UtcNow - Job.StartTime;
								Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, "[Maintain Jobs] Job \"" + JobName + "\" has been running for " + JobRunningTime.TotalSeconds.ToString() + " seconds" );

								// Update the job process stats
								try
								{
									Job.ProcessObjectPeakVirtualMemorySize64 = Job.ProcessObject.PeakVirtualMemorySize64;
									Job.ProcessObjectPeakWorkingSet64 = Job.ProcessObject.PeakWorkingSet64;
								}
								catch( Exception )
								{
									// Might throw an exception if the process exits while we're updating, it's okay
								}

								// If we're trying to avoid local execution, but the job is running and the
								// Instigator has no remote children to help but does have outstanding
								// reservations (presumably from local connections), ask the job check
								// the reservations to see if they should be rescheduled locally
								if( ( AgentApplication.Options.AvoidLocalExecution ) &&
									( Job.OwnerIsInstigator ) &&
									( Job.TaskReservationCount > 0 ) &&
									( Job.Owner.RemoteChildren.Count == 0 ) )
								{
									Job.CheckForReservations();
								}
							}
						}
						else if( Job.CurrentState == AgentJob.JobState.AGENT_JOB_CLOSED )
						{
							// Job has been marked closed, check for whether it quit cleanly on its own
							// as long as it's not the special debug job
							if( ( Job.ProcessObject != null ) &&
								( Job.ProcessObject.HasExited == false ) &&
								( Job.JobGuid.Equals( DebuggingJobGuid ) == false ) )
							{
								// Determine if the user-specified timeout has expired
								bool WaitedLongEnough = false;
								if( AgentApplication.DeveloperOptions.JobExecutableTimeout >= 0 )
								{
									// Wait for the user-specified time
                                    TimeSpan TimeSinceJobStopped = DateTime.UtcNow - Job.StopTime;
									TimeSpan TimeToWait = TimeSpan.FromSeconds(AgentApplication.DeveloperOptions.JobExecutableTimeout);

									if( TimeSinceJobStopped > TimeToWait )
									{
										string NewLogMessage = String.Format( "[Maintain Jobs] Waited for rogue Job for {0} seconds: \"{1}\"",
											AgentApplication.DeveloperOptions.JobExecutableTimeout,
											Job.JobGuid );
										Log( EVerbosityLevel.Informative, ELogColour.Orange, NewLogMessage );

										WaitedLongEnough = true;
									}
									else
									{
										Log( EVerbosityLevel.ExtraVerbose, ELogColour.Orange, "[Maintain Jobs] Waiting for Job to quit before killing: \"" + Job.JobGuid + "\"" );
									}
								}

								// If we've waited long enough or if the owner's parent is no longer
								// active (and thus unable to receive any of the results), kill it
								if( ( Job.Owner.CurrentState == ConnectionState.DISCONNECTED ) ||
									( WaitedLongEnough ) )
								{
									string NewLogMessage = String.Format( "[Maintain Jobs] Killing rogue Job \"{0}\"", Job.JobGuid );
									Log( EVerbosityLevel.Informative, ELogColour.Orange, NewLogMessage );

									Job.ProcessObject.Kill();

									// After killing the process, add the Job to the inactive list
									InactiveJobs.Add( Job.JobGuid );
								}
							}
							else
							{
								// Place it on the inactive list
								Log( EVerbosityLevel.Verbose, ELogColour.Green, "[Maintain Jobs] Job is closed and done, removing from list: \"" + Job.JobGuid + "\"" );
								InactiveJobs.Add( Job.JobGuid );
							}
						}
						else
						{
							Log( EVerbosityLevel.ExtraVerbose, ELogColour.Orange, "[Maintain Jobs] Job is " + Job.CurrentState.ToString() );
						}
					}
				}
			}
			// Remove all inactive Jobs
			foreach( AgentGuid NextGuid in InactiveJobs )
			{
				ActiveJobs.Remove( NextGuid );
			}
		}
	}
}

