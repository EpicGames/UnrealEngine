// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.Threading;

using AgentInterface;

namespace Agent
{
	///////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of the messaging subsystem in the Agent
	 */
	public partial class Agent : MarshalByRefObject, IAgentInternalInterface, IAgentInterface
	{
		///////////////////////////////////////////////////////////////////////////

		/**
		 * The agent's private message queues with SendMessage using SM, and
		 * ProcessMessage using PM, and swapping the two when we start a new
		 * loop through ProcessMessage
		 */
		private Queue<AgentMessage> MessageQueueSM = new Queue<AgentMessage>();
		private Queue<AgentMessage> MessageQueuePM = new Queue<AgentMessage>();
		private AutoResetEvent MessageQueueReady = new AutoResetEvent( false );
		private Object MessageQueueLock = new Object();

		/**
		 * Used to control access to the SendMessageInteranl routine, used
		 * when we need to ensure all messages have been processed
		 */
		ReaderWriterLock SendMessageLock = new ReaderWriterLock();

		///////////////////////////////////////////////////////////////////////////

		private class DisconnectionSignalMessage : AgentSignalMessage
		{
			public DisconnectionSignalMessage( Connection InConnectionToDisconnect )
			{
				ConnectionToDisconnect = InConnectionToDisconnect;
			}

			public Connection ConnectionToDisconnect;
		};

		///////////////////////////////////////////////////////////////////////////

		/**
		 * Tags incoming messages with the proper routing information and places it
		 * into the agent's private message queue. For all actual message processing,
		 * see ProcessMessage.
		 */
		private Int32 SendMessage_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "SendMessage_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;

			// Unpack the input parameters
			AgentMessage NewMessage = InParameters["Message"] as AgentMessage;

			// Treat standard calls to this API as "readers" only because we can
			// have many of them at a time
			SendMessageLock.AcquireReaderLock( Timeout.Infinite );

			// First, validate the connection being used to interact with the Swarm,
			// and only allow the message through if it's still alive
			Connection Sender;
			if( ( Connections.TryGetValue( ConnectionHandle, out Sender ) ) &&
				( Sender.CurrentState != ConnectionState.DISCONNECTED ) )
			{
				// Call through to the internal routine
				ErrorCode = SendMessageInternal( Sender, NewMessage );
			}
			else
			{
				string NewLogMessage = String.Format( "SendMessage: Message sent from unrecognized sender ({0:X8}), ignoring \"{1}\"", ConnectionHandle, NewMessage.Type );
				Log( EVerbosityLevel.Verbose, ELogColour.Orange, NewLogMessage );
                ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			// Release our "reader" lock
			SendMessageLock.ReleaseReaderLock();

            StopTiming();
            return ( ErrorCode );
		}

		/**
		 * Actual work performed by SendMessage happens here
		 */
		public Int32 SendMessageInternal( Connection Sender, AgentMessage NewMessage )
		{
			Int32 ErrorCode = Constants.INVALID;

			// We assume the message is valid, but if somewhere below we change our
			// mind, this value will be set to false and we'll eat the message
			bool bMessageIsValid = true;

			// Logic for the setting of the To and From fields (if not already set)
			// 
			// All connections sending messages are implicitly sending them to the
			// Instigator of the Job they're working on. Depending on where the
			// message is coming from, we might need to patch up the To field a
			// little bit to ensure it's heading to a Local connection when it
			// needs to be. The only case this applies is when we receive a message
			// from a Remote connection, directed toward a Remote connection, which
			// happens when we get a message from a Remote Agent from an even more
			// Remote connection (i.e. a Job connection on the remote machine):
			// 
			// To Local, From Local -> routing of main message and replies are ok
			// To Local, From Remote -> routing of main message and replies are ok
			// To Remote, From Local -> routing of main message and replies are ok
			// To Remote, From Remote -> routing of replies is ok, but the main
			//      message is in trouble if it's not completely handled within the
			//      Agent's ProcessMessages routine. It would be forwarded on to the
			//      To field connection which is Remote and we'd end up bouncing the
			//      message back and forth forever. Need to adjust the To field to
			//      point to the parent of the To connection (which is where it's
			//      intended to go anyway). See further below for where we handle
			//      this case.

			// If the From field is not set, give it the connection handle value by
			// default so that any response message will automatically be routed back
			// to it whether it's the sender or the recipient
			if( NewMessage.From == Constants.INVALID )
			{
				NewMessage.From = Sender.Handle;
			}
			// If the connection used to send the message is Remote, check for the
			// Remote -> Remote case described above
			else if( Sender is RemoteConnection )
			{
				// If the From field is already set, see if we've already registered
				// the connection this message is being sent from
				Connection FromConnection;
				if( !Connections.TryGetValue( NewMessage.From, out FromConnection ) )
				{
					// This is a new one - make it an alias for the sender so that any
					// responses will be directed back via its sending interface
					RemoteConnection RemoteSender = Sender as RemoteConnection;
					RemoteSender.Aliases.Add( NewMessage.From );
					// There are times we want to ensure no new connections are coming online
					// where we'll lock the Connections dictionary (see MaintainCache)
					lock( Connections )
					{
						Connections.Add( NewMessage.From, RemoteSender );
					}
					FromConnection = RemoteSender;

					string LogMessage = String.Format( "[SendMessage] Added alias for remote connection: {0:X8} is an alias for {1:X8}",
						NewMessage.From,
						Sender.Handle );
					Log( EVerbosityLevel.Informative, ELogColour.Green, LogMessage );
				}

				// If this is a Remote -> Remote situation, the proper place to route
				// the message to the parent of the remote connection since the Agents
				// generally act as glue between connections
				if( FromConnection is RemoteConnection )
				{
					Debug.Assert( NewMessage.To != Constants.INVALID );

					Connection ToConnection;
					if( (Connections.TryGetValue( NewMessage.To, out ToConnection )) &&
						(ToConnection is RemoteConnection) )
					{
						Connection ToConnectionParent = ToConnection.Parent;
						if( ToConnectionParent != null )
						{
							NewMessage.To = ToConnectionParent.Handle;
						}
					}
				}
			}

			// If the To field is not set, assign it based on the message type
			if( NewMessage.To == Constants.INVALID )
			{
				// TODO: As we add additional versions, convert to a switch rather than if-else.
				// For now, just use a simple if since we only have one version and a switch is
				// overkill.
				if( NewMessage.Version == ESwarmVersionValue.VER_1_0 )
				{
					// The default is for messages to be ultimately routed to the Instigator
					// unless the message is one of a certain set of types that route
					// directly to the connection specified
					switch( NewMessage.Type )
					{
						// These message types need to be routed to the connection specified
						// either because they are meant to be simple round-trip messages or
						// because they are sent from within Swarm directly to the connection
						// and should not be routed anywhere else

						case EMessageType.QUIT:
						case EMessageType.PING:
						case EMessageType.SIGNAL:
							NewMessage.To = Sender.Handle;
							break;

						// These message types need to be routed eventually to the Instigator
						// connection, which is the ultimate ancestor up the parent chain, so
						// simply assign the most senior parent we have

						case EMessageType.INFO:
						case EMessageType.ALERT:
						case EMessageType.TIMING:
						case EMessageType.TASK_REQUEST:
						case EMessageType.TASK_STATE:
						case EMessageType.JOB_STATE:
							// By default, make the sender the recipient for these cases, in
							// case the parent is no longer active
							NewMessage.To = Sender.Handle;
							Connection SenderParent = Sender.Parent;
							if( SenderParent != null )
							{
								// If we have a parent connection and it's active, then
								// assign it as the recipient
								if( SenderParent.CurrentState == ConnectionState.CONNECTED )
								{
									NewMessage.To = SenderParent.Handle;
								}
							}
							break;

						// These message types are not expected and are each error cases

						case EMessageType.NONE:						// Should never be set to this
						case EMessageType.JOB_SPECIFICATION:		// Only used for messages going directly into OpenJob
						case EMessageType.TASK_REQUEST_RESPONSE:	// Should always have the To field set already
						default:
                            Log( EVerbosityLevel.Informative, ELogColour.Orange, "SendMessage: Invalid message type received, ignoring " + NewMessage.Type.ToString() );
							break;
					}

					// If still no assigned To field, consider it an error
					if( NewMessage.To == Constants.INVALID )
					{
                        Log( EVerbosityLevel.Informative, ELogColour.Orange, "SendMessage: No proper recipient found, ignoring " + NewMessage.Type.ToString() );
						bMessageIsValid = false;
					}
				}
			}

			// If the message remains valid, post it to the queue
			if( bMessageIsValid )
			{
				lock( MessageQueueLock )
				{
					Debug.Assert( NewMessage != null );
					MessageQueueSM.Enqueue( NewMessage );

					string NewLogMessage = String.Format( "Step 1 of N for message: ({0:X8} -> {1:X8}), {2}, Message Count {3} (Agent)",
						NewMessage.To,
						NewMessage.From,
						NewMessage.Type,
						MessageQueueSM.Count );

					Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, NewLogMessage );
					
					MessageQueueReady.Set();
				}
				ErrorCode = Constants.SUCCESS;
			}
			else
			{
				Log( EVerbosityLevel.Informative, ELogColour.Orange, String.Format( "SendMessage: Discarded message \"{0}\"", NewMessage.Type ) );
			}

			return ( ErrorCode );
		}

		/*
		 * Get the machine name from the connection structure
		 */
		public string MachineNameFromConnection( Connection RequestingConnection )
		{
			string Name = "";

			RemoteConnection Remote = RequestingConnection as RemoteConnection;
			if( Remote == null )
			{
				Name = System.Net.Dns.GetHostName();
			}
			else
			{
				Name = Remote.Info.Name;
			}

			return ( Name );
		}

		/*
		 * Get the machine IP address from the connection structure
		 */
		public string MachineIPAddressFromConnection( Connection RequestingConnection )
		{
			string MachineIPAddress = "";

			RemoteConnection Remote = RequestingConnection as RemoteConnection;
			if( Remote == null )
			{
				MachineIPAddress = "127.0.0.1";
			}
			else
			{
				MachineIPAddress = Remote.Interface.RemoteAgentIPAddress;
			}

			return ( MachineIPAddress );
		}

		/**
		 * Flush the message queue - note this does not assure that it's empty, it just
		 * makes sure that all messages that are in the queue at the time of this method
		 * call are processed. This call is blocking.
		 */
		public bool FlushMessageQueue( Connection RequestingConnection, bool WithDisconnect )
		{
			Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, String.Format( "[FlushMessageQueue] Draining message queue for {0:X8}", RequestingConnection.Handle ) );

			// This special call will act as a "writer" since we only want to
			// allow one of these to act at a time. By doing this, we ensure
			// that all "readers" that use this lock are finished.
			SendMessageLock.AcquireWriterLock( Timeout.Infinite );
			Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, String.Format( "[FlushMessageQueue] Lock acquired for {0:X8}", RequestingConnection.Handle ) );

			// Flush the message queue with a signal message, optionally with a disconnection event
			AgentSignalMessage SignalMessage;
			if( WithDisconnect )
			{
				SignalMessage = new DisconnectionSignalMessage( RequestingConnection );
			}
			else
			{
				SignalMessage = new AgentSignalMessage();
			}
			Int32 SignalMessageSent = SendMessageInternal( RequestingConnection, SignalMessage );

			// Release our "writer" lock
			SendMessageLock.ReleaseWriterLock();
			Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, String.Format( "[FlushMessageQueue] Lock released for {0:X8}", RequestingConnection.Handle ) );

			if( SignalMessageSent == Constants.SUCCESS )
			{
				SignalMessage.ResetEvent.WaitOne( Timeout.Infinite );
				Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, String.Format( "[FlushMessageQueue] Drain complete for {0:X8}", RequestingConnection.Handle ) );
				return true;
			}
			else
			{
				Log( EVerbosityLevel.ExtraVerbose, ELogColour.Green, "[FlushMessageQueue] Drain failed because of an error sending the signal message" );
				return false;
			}
		}

		/**
		 * Takes messages off the internal message queue and handles them by either
		 * sending responses, forwarding the message on, or processing it internally
		 */
		private void ProcessMessagesThreadProc()
		{
			// A response message queue used to send messages back to the one which sent it
			Queue<AgentMessage> ResponseMessageQueue = new Queue<AgentMessage>();

			while( AgentHasShutDown == false )
			{
				StartTiming( "ProcessMessage-Internal", true );

				lock( MessageQueueLock )
				{
					// Swap the SM and PM message queue to keep things moving at full speed
					Queue<AgentMessage> Temp = MessageQueuePM;
					MessageQueuePM = MessageQueueSM;
					MessageQueueSM = Temp;
				}

				// Process all messages currently in the queue
				while( MessageQueuePM.Count > 0 )
				{
					Debug.Assert( ResponseMessageQueue.Count == 0 );

					// Get and process the next message
					AgentMessage NextMessage = MessageQueuePM.Dequeue();
					Debug.Assert( NextMessage != null );

					bool bMessageHandled = false;
					switch( NextMessage.Type )
					{
						case EMessageType.SIGNAL:
						{
							if( NextMessage is DisconnectionSignalMessage )
							{
								// Mark the connection as inactive
								DisconnectionSignalMessage DisconnectMessage = NextMessage as DisconnectionSignalMessage;
								Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[CloseConnection] Connection disconnected {0:X8}", DisconnectMessage.ConnectionToDisconnect.Handle ) );
								DisconnectMessage.ConnectionToDisconnect.CurrentState = ConnectionState.DISCONNECTED;
                                DisconnectMessage.ConnectionToDisconnect.DisconnectedTime = DateTime.UtcNow;
							}

							// Signal the message and move on
							AgentSignalMessage SignalMessage = NextMessage as AgentSignalMessage;
							SignalMessage.ResetEvent.Set();
							bMessageHandled = true;
						}
						break;

						case EMessageType.TIMING:
						{
							Connection FromConnection;
							if( ( Connections.TryGetValue( NextMessage.From, out FromConnection ) ) )
							{
								Connection ToConnection;
								if( ( Connections.TryGetValue( NextMessage.To, out ToConnection ) ) &&
									( ToConnection is LocalConnection ) )
								{
									// Handle message
									AgentTimingMessage TimingMessage = NextMessage as AgentTimingMessage;
									AgentApplication.UpdateMachineState( MachineNameFromConnection( FromConnection ), TimingMessage.ThreadNum, TimingMessage.State );
									bMessageHandled = true;
								}
							}
						}
						break;

						case EMessageType.TASK_REQUEST:
						{
							// Look up the requesting connection
							Debug.Assert( NextMessage.From != Constants.INVALID );
							Connection RequestingConnection;
							if( Connections.TryGetValue( NextMessage.From, out RequestingConnection ) )
							{
								// Look up the specified Job
								AgentJob JobToAskForTasks = RequestingConnection.Job;
								if( JobToAskForTasks != null )
								{
									// If we get a valid response back, add it to the queue
									AgentTaskRequestResponse Response = JobToAskForTasks.GetNextTask( RequestingConnection );
									if( Response != null )
									{
										ResponseMessageQueue.Enqueue( Response );
										
										// Specifications and releases are always handled here, but
										// reservations are special in that we will send a reservation
										// back to local connections but we'll need to make sure the
										// message continues on to remote connections.
										if( ( Response.ResponseType == ETaskRequestResponseType.SPECIFICATION ) ||
											( Response.ResponseType == ETaskRequestResponseType.RELEASE ) ||
											( ( Response.ResponseType == ETaskRequestResponseType.RESERVATION ) &&
											  ( JobToAskForTasks.Owner is LocalConnection ) ) )
										{
											bMessageHandled = true;
										}
									}
								}
								else
								{
									// Unable to find the Job, just send back a release message
									Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[ProcessMessage] Unable to find Job for Task Request; may have been closed" );
									//ResponseMessageQueue.Enqueue( new AgentTaskRequestResponse( RequestingConnection.Job.JobGuid,
									//															ETaskRequestResponseType.RELEASE ) );
									bMessageHandled = true;
								}
							}
							else
							{
								// Unable to find the connection, swallow the request
								Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[ProcessMessage] Unable to find owning Connection for Task Request" );
								bMessageHandled = true;
							}
						}
						break;

						case EMessageType.TASK_STATE:
						{
							// Look up the sending connection
							Debug.Assert( NextMessage.From != Constants.INVALID );
							Connection SendingConnection;
							if( ( Connections.TryGetValue( NextMessage.From, out SendingConnection ) ) &&
								( SendingConnection.Job != null ) )
							{
								// Look up the specified Job
								AgentJob UpdatedJob;
								if( ActiveJobs.TryGetValue( SendingConnection.Job.JobGuid, out UpdatedJob ) )
								{
									AgentTaskState UpdatedTaskState = NextMessage as AgentTaskState;
									UpdatedJob.UpdateTaskState( UpdatedTaskState );

									if( UpdatedJob.Owner is LocalConnection )
									{
										// If the Task state change is of a type potentially interesting to
										// the Instigator, return it
										switch( UpdatedTaskState.TaskState )
										{
											case EJobTaskState.TASK_STATE_INVALID:
											case EJobTaskState.TASK_STATE_COMPLETE_SUCCESS:
											case EJobTaskState.TASK_STATE_COMPLETE_FAILURE:
												// For these message types, allow the message to continue on
												break;

											default:
												// Nothing to do otherwise, mark the message as handled
												bMessageHandled = true;
												break;
										}
									}
									else
									{
										// Always send messages on for remote connections
									}
								}
								else
								{
									// Unable to find the Job, swallow the request
									Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[ProcessMessage] Unable to find Job for Task Request" );
									bMessageHandled = true;
								}
							}
							else
							{
								// Unable to find the connection, swallow the request
								Log( EVerbosityLevel.Verbose, ELogColour.Orange, "[ProcessMessage] Unable to find owning Connection for Task Request" );
								bMessageHandled = true;
							}
						}
						break;
					}

					// If the message was not handled completely, send it on
					if( bMessageHandled == false )
					{
						// Look up who the message is being sent to and make sure they're
						// still active and if not, ignore the message
						Connection Recipient;
						Debug.Assert( NextMessage.To != Constants.INVALID );
						if( Connections.TryGetValue( NextMessage.To, out Recipient ) )
						{
							if( Recipient is LocalConnection )
							{
								// If the recipient is local, place it in the proper queue
								// and signal that a message is ready
								LocalConnection LocalRecipient = Recipient as LocalConnection;
								lock( LocalRecipient.MessageQueue )
								{
									LocalRecipient.MessageQueue.Enqueue( NextMessage );

									string NewLogMessage = String.Format( "Step 2 of 4 for message: ({0:X8} -> {1:X8}), {2}, Message Count {3} (Local Connection)",
										NextMessage.To,
										NextMessage.From,
										NextMessage.Type,
										LocalRecipient.MessageQueue.Count );

									Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, NewLogMessage );

									LocalRecipient.MessageAvailableSignal();
								}
							}
							else
							{
								Debug.Assert( Recipient is RemoteConnection );

								// If the recipient is remote, send the message via SendMessage
								// unless the message is a Task being sent back, which is sent
								// via the dedicated Task API
								RemoteConnection RemoteRecipient = Recipient as RemoteConnection;
								if( NextMessage is AgentTaskSpecification )
								{
									// All new Tasks are sent via the dedicated Task API
									AgentTaskSpecification TaskSpecification = NextMessage as AgentTaskSpecification;

									Hashtable RemoteInParameters = new Hashtable();
									RemoteInParameters["Version"] = ESwarmVersionValue.VER_1_0;
									RemoteInParameters["Specification"] = TaskSpecification;
									Hashtable RemoteOutParameters = null;

									Int32 Error = RemoteRecipient.Interface.AddTask( RemoteRecipient.Handle, RemoteInParameters, ref RemoteOutParameters );
									if( Error >= 0 )
									{
										// Perhaps we should be sending an accept message back?
									}
									else
									{
										AgentTaskState UpdateMessage;
										if( Error == Constants.ERROR_CONNECTION_DISCONNECTED )
										{
											// Special case of the connection dropping while we're adding the
											// task, say it's been killed to requeue
											UpdateMessage = new AgentTaskState( TaskSpecification.JobGuid,
																				TaskSpecification.TaskGuid,
																				EJobTaskState.TASK_STATE_KILLED );
										}
										else
										{
											// All other error cases will be rejections
											UpdateMessage = new AgentTaskState( TaskSpecification.JobGuid,
																				TaskSpecification.TaskGuid,
																				EJobTaskState.TASK_STATE_REJECTED );
										}
										AgentJob Job;
										if( ActiveJobs.TryGetValue( TaskSpecification.JobGuid, out Job ) )
										{
											Job.UpdateTaskState( UpdateMessage );
										}
									}
								}
								else
								{
									// All standard messages are sent via the SendMessage API
									Hashtable RemoteInParameters = new Hashtable();
									RemoteInParameters["Version"] = ESwarmVersionValue.VER_1_0;
									RemoteInParameters["Message"] = NextMessage;
									Hashtable RemoteOutParameters = null;

									RemoteRecipient.Interface.SendMessage( NextMessage.To, RemoteInParameters, ref RemoteOutParameters );
								}

								string NewLogMessage = String.Format( "Step 2 of 2 for message: ({0:X8} -> {1:X8}), {2}, (Remote Connection)",
									NextMessage.To,
									NextMessage.From,
									NextMessage.Type );

								Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, NewLogMessage );
							}
						}
						else
						{
                            Log( EVerbosityLevel.Informative, ELogColour.Orange, "ProcessMessage: Message sent to invalid connection, ignoring: " + NextMessage.Type.ToString() );
						}
					}

					// If there are any responses to the message, send them
					if( ResponseMessageQueue.Count > 0 )
					{
						foreach( AgentMessage NextResponse in ResponseMessageQueue )
						{
							// For each one of the messages, set the routing fields properly
							NextResponse.To = NextMessage.From;
							NextResponse.From = NextMessage.To;

							// And then queue the message back up immediately
							MessageQueuePM.Enqueue( NextResponse );
						}
						ResponseMessageQueue.Clear();
					}
				}

                StopTiming();

				// Wait for a message to become available and once unlocked, swap the queues
				// and check for messages to process. Set a timeout, so we'll wake up every
				// now and then to check for a quit signal at least
				MessageQueueReady.WaitOne( 500 );
			}
		}

		/**
		 * Gets the next message in the queue, optionally blocking until a message is available
		 */
		private Int32 GetMessage_1_0( Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters )
		{
			StartTiming( "GetMessage_1_0-Internal", true );

			Int32 ErrorCode = Constants.INVALID;
			AgentMessage NextMessage = null;

			// Unpack the input parameters
			Int32 Timeout = ( Int32 )InParameters["Timeout"];

			// First, validate the connection requesting the message by making sure it
			// exists and is local
			Connection Recipient;
			if( ( Connections.TryGetValue( ConnectionHandle, out Recipient ) ) &&
				( Recipient is LocalConnection ) )
			{	
				// This should only ever be called by local connections since remote
				// connections have a direct messaging link between Agents
				LocalConnection ConnectionMessageSentTo = Recipient as LocalConnection;

				StopTiming();
                
				// If no message is available, wait for one for the specified amount of time
				bool Signaled = ConnectionMessageSentTo.MessageAvailableWait( Timeout );

				StartTiming( "GetMessage-Internal", true );

				// If the semaphore was signaled, then check the queue and get the next message
				if( Signaled )
				{
					// Thread-safe queue usage
					lock( ConnectionMessageSentTo.MessageQueue )
					{
						// Check to see if we actually have any messages to dequeue (standard case)
						if( ConnectionMessageSentTo.MessageQueue.Count > 0 )
						{
							// Dequeue and process the message
							NextMessage = ConnectionMessageSentTo.MessageQueue.Dequeue();

							OutParameters = new Hashtable();
							OutParameters["Version"] = ESwarmVersionValue.VER_1_0;
							OutParameters["Message"] = NextMessage;

							string NewLogMessage = String.Format( "Step 3 of 4 for message: ({0:X8} -> {1:X8}), {2}, Message Count {3} (Local Connection)",
								NextMessage.To,
								NextMessage.From,
								NextMessage.Type,
								ConnectionMessageSentTo.MessageQueue.Count );

							Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, NewLogMessage );
							ErrorCode = Constants.SUCCESS;
						}
						else
						{
							// If the semaphore was signaled and the message queue is empty, which
							// is only true after CloseConnection is called, we're done
							ErrorCode = Constants.ERROR_CONNECTION_DISCONNECTED;
						}
					}
				}
				else
				{
					// Otherwise, the timeout period has elapsed; this is still considered a
					// success only without a message to send back
					Log( EVerbosityLevel.Verbose, ELogColour.Green, String.Format( "[GetMessage] Message available timeout expired, safely returning to {0:X8} with no message", ConnectionHandle ) );
					ErrorCode = Constants.SUCCESS;
				}
			}
			else
			{
				string LogMessage = String.Format( "[GetMessage] Either connection doesn't exist or is not a local connection: {0:X8}", ConnectionHandle );
				Log( EVerbosityLevel.Informative, ELogColour.Red, LogMessage );
                ErrorCode = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			// Determine if we have a message to send back
			if( NextMessage != null )
			{
				string NewLogMessage = String.Format( "Step 4 of 4 for message: ({0:X8} -> {1:X8}), {2}, Delivered (Local Connection)",
					NextMessage.To,
					NextMessage.From,
					NextMessage.Type );
				Log( EVerbosityLevel.SuperVerbose, ELogColour.Green, NewLogMessage );
			}
			else if( Timeout == -1 )
			{
				if( ( Recipient != null ) &&
					( Recipient.CurrentState == ConnectionState.DISCONNECTED ) )
				{
					Log( EVerbosityLevel.Informative, ELogColour.Green, String.Format( "[GetMessage] Safely returning to {0:X8} with no message", ConnectionHandle ) );
				}
				else
				{
					Log( EVerbosityLevel.Informative, ELogColour.Orange, String.Format( "[GetMessage] Returning to {0:X8} with no message, possibly in error", ConnectionHandle ) );
				}
			}

            StopTiming();
            return ( ErrorCode );
		}
	}
}

