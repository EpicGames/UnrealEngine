// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Drawing;
using System.Windows.Forms;

using AgentInterface;

namespace Agent
{
	/*
	 * Everything required to handle storing the data for drawing the progress bars
	 */
	public partial class SwarmAgentWindow : Form
	{
		class BarColour
		{
			public Color Colour;
			public Color BorderColour;
			public Pen Border;
			public SolidBrush FillBrush;

			public BarColour( Color InColour )
			{
				Colour = InColour;
				BorderColour = Color.Black;

				Border = new Pen( BorderColour );
				FillBrush = new SolidBrush( Colour );
			}
		}

		private static Dictionary<string, BarColour> BarColours = null;
		private static Dictionary<string, string> BarNames = null;

		public static void CreateBarColours( SwarmAgentWindow Owner )
		{
			BarColours = new Dictionary<string, BarColour>();
			for( int Index = 0; Index < ( int )EProgressionState.Num; Index++ )
			{
				BarColours.Add( Enum.GetName( typeof( EProgressionState ), Index ), new BarColour( AgentApplication.Options.VisualizerColors[Index] ) );
			}

			// Refresh the key
			Owner.ExportingBox.BackColor = AgentApplication.Options.VisualizerColors[( int )EProgressionState.InstigatorConnected];
			Owner.StartingBox.BackColor = AgentApplication.Options.VisualizerColors[( int )EProgressionState.BeginJob];
			Owner.EmitPhotonBox.BackColor = AgentApplication.Options.VisualizerColors[( int )EProgressionState.Preparing0];
			Owner.IrradianceCacheBox.BackColor = AgentApplication.Options.VisualizerColors[( int )EProgressionState.Preparing2];
			Owner.ProcessingBox.BackColor = AgentApplication.Options.VisualizerColors[( int )EProgressionState.Processing0];
			Owner.ExportingResultsBox.BackColor = AgentApplication.Options.VisualizerColors[( int )EProgressionState.ExportingResults];
			Owner.ImportingBox.BackColor = AgentApplication.Options.VisualizerColors[( int )EProgressionState.Finished];

			BarNames = new Dictionary<string, string>();
			BarNames.Add( EProgressionState.TaskTotal.ToString(),				"Task total" );
			BarNames.Add( EProgressionState.TasksInProgress.ToString(),			"Tasks in progress" );
			BarNames.Add( EProgressionState.TasksCompleted.ToString(),			"Tasks completed" );
			BarNames.Add( EProgressionState.Idle.ToString(),					"Idle" );
			BarNames.Add( EProgressionState.InstigatorConnected.ToString(),		"Exporting to local cache" );
			BarNames.Add( EProgressionState.RemoteConnected.ToString(),			"Connected to remote, starting job" );
			BarNames.Add( EProgressionState.Exporting.ToString(),				"Exporting to local cache" );
			BarNames.Add( EProgressionState.BeginJob.ToString(),				"Lightmass starting up" );
			BarNames.Add( EProgressionState.Blocked.ToString(),					"Waiting for remote to become available" );
			BarNames.Add( EProgressionState.Preparing0.ToString(),				"Emitting direct photons" );
			BarNames.Add( EProgressionState.Preparing1.ToString(),				"Emitting indirect photons" );
			BarNames.Add( EProgressionState.Preparing2.ToString(),				"Calculating irradiance photons" );
			BarNames.Add( EProgressionState.Preparing3.ToString(),				"Caching irradiance photons" );
			BarNames.Add( EProgressionState.Processing0.ToString(),				"Processing mappings" );
			BarNames.Add( EProgressionState.FinishedProcessing0.ToString(),		"Finished processing mappings" );
			BarNames.Add( EProgressionState.Processing1.ToString(),				"Processing1" );
			BarNames.Add( EProgressionState.FinishedProcessing1.ToString(),		"Finished processing1" );
			BarNames.Add( EProgressionState.Processing2.ToString(),				"Processing2" );
			BarNames.Add( EProgressionState.FinishedProcessing2.ToString(),		"Finished processing2" );
			BarNames.Add( EProgressionState.Processing3.ToString(),				"Processing3" );
			BarNames.Add( EProgressionState.FinishedProcessing3.ToString(),		"Finished processing3" );
			BarNames.Add( EProgressionState.ExportingResults.ToString(),		"Exporting results" );
			BarNames.Add( EProgressionState.ImportingResults.ToString(),		"Importing results" );
			BarNames.Add( EProgressionState.Finished.ToString(),				"Finishing" );
			BarNames.Add( EProgressionState.RemoteDisconnected.ToString(),		"Remote disconnected" );
			BarNames.Add( EProgressionState.InstigatorDisconnected.ToString(),	"Instigator disconnected" );
            BarNames.Add( EProgressionState.Preparing4.ToString(),              "Skylight Radiosity");
		}

		public class ProgressionEvent
		{
			public string Machine = null;
			public int ThreadNum = 0;
			public EProgressionState State = EProgressionState.Idle;
			public DateTime Time = DateTime.UtcNow;

			public ProgressionEvent( string InMachine, int InThreadNum, EProgressionState NewState )
			{
				Machine = InMachine;
				ThreadNum = InThreadNum;
				State = NewState;
			}
		}

		public class ProgressionThreadSample
		{
			public EProgressionState State = EProgressionState.Idle;
			public DateTime StartTime = DateTime.UtcNow;
			public DateTime EndTime = DateTime.UtcNow;
			public int NumTimestamps = 0;

			public ProgressionThreadSample(EProgressionState NewState)
			{
				State = NewState;
				NumTimestamps = 0;
			}

			public void AddTimestamp( DateTime EventTime )
			{
				if ( NumTimestamps == 0 )
				{
					StartTime = EventTime;
				}
				EndTime = EventTime;
				NumTimestamps++;
			}
		}

		public class ProgressionSample
		{
			public EProgressionState State = EProgressionState.Idle;
			public DateTime StartTime = DateTime.UtcNow;
			public DateTime EndTime = DateTime.UtcNow;
			public List<ProgressionThreadSample> ThreadSamples = null;
			public Rectangle Bounds;
			public double Duration;
			public int NumTimestamps = 0;

			public ProgressionSample()
			{
			}

			public ProgressionSample( EProgressionState NewState, DateTime EventTime )
			{
				State = NewState;
				StartTime = EventTime;
				EndTime = EventTime;
				Duration = 0.0;
				NumTimestamps = 1;
			}

			public void AddTimestamp( DateTime EventTime )
			{
				if ( NumTimestamps == 0 )
				{
					StartTime = EventTime;
				}
				EndTime = EventTime;
				NumTimestamps++;
			}
		}

		public class Progression
		{
			/*
			 * List of state changes and times thereof for all threads
			 */
			public List<ProgressionSample> ProgressionSamples = new List<ProgressionSample>();

			private const int BarBorder = 4;

			/** Total width of all progress bars (within the 2nd column). */
			public int ProgressTotalWidth = 0;

			public Progression()
			{
				ProgressionSamples.Add( new ProgressionSample() );
				ProgressionSamples.Add( new ProgressionSample() );
			}

			public void DrawBackground( Graphics Gfx, Rectangle Bounds )
			{
				Color FillColor = SystemColors.ControlLightLight;
				ProgressionSample Sample = ProgressionSamples[ProgressionSamples.Count - 1];
                if (Sample.State == EProgressionState.RemoteDisconnected || Sample.State == EProgressionState.InstigatorDisconnected)
				{
					FillColor = SystemColors.ControlLight;
				}

				Gfx.Clear( FillColor );
			}

			private void DrawFilledBar( Graphics Gfx, EProgressionState State, int Left, int Top, int Width, int Height )
			{
				Rectangle Bar = new Rectangle( Left, Top, Width, Height );

				Gfx.FillRectangle( BarColours[State.ToString()].FillBrush, Bar );

				Pen Border = BarColours[State.ToString()].Border;
				Gfx.DrawRectangle( Border, Bar );
			}

			public int Draw( DateTime Start, float ZoomLevel, Graphics Gfx, Rectangle Bounds )
			{
				int Rightmost = 0;
				int Offset = BarBorder + Bounds.Left;
				// Work out initial offset so all the bars are on the same time rule
				ProgressionSample LastSample = ProgressionSamples[ProgressionSamples.Count - 1];
				int Top = Bounds.Top + BarBorder;
				int Height = Bounds.Height - ( BarBorder * 2 );

				for( int BarIndex = 1; BarIndex < ProgressionSamples.Count; BarIndex++ )
				{
					ProgressionSample NewSample = ProgressionSamples[BarIndex];

					// Render progress bars
					if( NewSample.ThreadSamples == null )
					{
						int Left = ( int )( ( NewSample.StartTime - Start ).TotalSeconds * ZoomLevel );
						int Right = ( int )( ( NewSample.EndTime - Start ).TotalSeconds * ZoomLevel );
						if( Right - Left > 2 )
						{
							DrawFilledBar( Gfx, NewSample.State, Left + Offset, Top, Right - Left, Height );
							NewSample.Bounds = new Rectangle( Left + Offset, Top, Right - Left, Height );
							NewSample.Duration = (NewSample.EndTime - NewSample.StartTime).TotalSeconds;

							if( Right > Rightmost )
							{
								Rightmost = Right;
							}
						}
					}
					else
					{
						// Render a line per thread with ticks for new jobs
						int HeightPerThread = Math.Max(Height / NewSample.ThreadSamples.Count, 2);
						int ThreadTop = Top;
                        int AccumulatedThreadHeight = 0;
						int BarThreadLeft = 0;
						int BarThreadRight = 0;
						int BarThreadLeftMost = Int32.MaxValue;
						int BarThreadRightMost = 0;

						foreach( ProgressionThreadSample ThreadSample in NewSample.ThreadSamples )
						{
                            if (AccumulatedThreadHeight < Height)
                            {
                                // Draw the thread line
                                BarThreadLeft = (int)(ZoomLevel * (ThreadSample.StartTime - Start).TotalSeconds);

                                if (BarIndex < (ProgressionSamples.Count - 1) || ThreadSample.StartTime != ThreadSample.EndTime)
                                {
                                    BarThreadRight = (int)(ZoomLevel * (ThreadSample.EndTime - Start).TotalSeconds);
                                }
                                else if (LastSample.State == EProgressionState.RemoteDisconnected || LastSample.State == EProgressionState.InstigatorDisconnected)
                                {
                                    BarThreadRight = (int)(ZoomLevel * (LastSample.StartTime - Start).TotalSeconds);
                                }
                                else
                                {
                                    BarThreadRight = (int)(ZoomLevel * (DateTime.UtcNow - Start).TotalSeconds);
                                }

                                // Make sure each bar is at least 1 unit wide, and maintain extrema.
                                if (BarThreadRight <= BarThreadLeft)
                                {
                                    BarThreadRight = BarThreadLeft + 1;
                                }
                                if (BarThreadLeft < BarThreadLeftMost)
                                {
                                    BarThreadLeftMost = BarThreadLeft;
                                }
                                if (BarThreadRight > BarThreadRightMost)
                                {
                                    BarThreadRightMost = BarThreadRight;
                                }

                                Rectangle ThreadBar = new Rectangle(BarThreadLeft + Offset, ThreadTop + 1, BarThreadRight - BarThreadLeft, HeightPerThread - 1);
                                BarColour ThreadBarColor = BarColours[ThreadSample.State.ToString()];
                                Gfx.FillRectangle(ThreadBarColor.FillBrush, ThreadBar);

                                ThreadTop += HeightPerThread;
                                AccumulatedThreadHeight += HeightPerThread;
                            }
						}

						// Add a border
						NewSample.Bounds = new Rectangle( BarThreadLeftMost + Offset, Top, BarThreadRightMost - BarThreadLeftMost, Height );
						NewSample.Duration = NewSample.Bounds.Width / ZoomLevel;
						Gfx.DrawRectangle( Pens.Black, NewSample.Bounds );

						if( BarThreadRightMost > Rightmost )
						{
							Rightmost = BarThreadRightMost;
						}
					}
				}

				ProgressTotalWidth = Rightmost;
				return ( Rightmost );
			}
		}

		public class Progressions
		{
			/*
			 * Overall progression status
			 */
			public int NumTasks = 0;
			public int NumRetiredTasks = 0;
			public int NumRunningTasks = 0;

			/*
			 * The time the instigator connected
			 */
			public DateTime StartTime = DateTime.UtcNow;

			/*
			 * The collection that contains the timing info per machine
			 * 
			 * Each entry in the list represents a thread
			 */
			public ReaderWriterDictionary<string, Progression> MachineProgressions = new ReaderWriterDictionary<string, Progression>();

			public void DrawOverallProgressBar( Graphics Gfx, Rectangle Bounds )
			{
				Gfx.Clear( Color.White );
				
				float PercentComplete = 0.0f;
				float PercentRunning = 0.0f;
				if( NumTasks > 0 )
				{
					PercentComplete = ( float )NumRetiredTasks / ( float )NumTasks;
					PercentRunning = ( float )NumRunningTasks / ( float )NumTasks;
				}

				Rectangle CompleteBar = new Rectangle( Bounds.X, Bounds.Y, ( int )( Bounds.Width * PercentComplete ) + 1, Bounds.Height );
				Rectangle RunningBar = new Rectangle( Bounds.X + ( int )( Bounds.Width * PercentComplete ) + 1, Bounds.Y, ( int )( Bounds.Width * PercentRunning ) + 1, Bounds.Height );

				Gfx.FillRectangle( BarColours[EProgressionState.FinishedProcessing3.ToString()].FillBrush, CompleteBar );
				Gfx.FillRectangle( BarColours[EProgressionState.FinishedProcessing2.ToString()].FillBrush, RunningBar );

				Pen Border = BarColours[EProgressionState.BeginJob.ToString()].Border;
				Rectangle BorderBounds = new Rectangle( Bounds.X, Bounds.Y, Bounds.Width - 1, Bounds.Height - 1 );
				Gfx.DrawRectangle( Border, BorderBounds );

				string ProgressString = ( PercentComplete * 100.0f ).ToString( "F2" ) + "%";
				Font DrawFont = new Font( AgentApplication.Options.TextFont.ToString(), 9.75F );
				SizeF StringSize = Gfx.MeasureString( ProgressString, DrawFont );
				int X = BorderBounds.X + ( BorderBounds.Width - ( int )StringSize.Width ) / 2;
				int Y = BorderBounds.Y + ( BorderBounds.Height - ( int )StringSize.Height ) / 2;
				Gfx.DrawString( ProgressString, DrawFont, Brushes.Black, X, Y );
				DrawFont.Dispose();
			}

			/*
			 * Process a state change timing event
			 */
			public bool ProcessEvent( ProgressionEvent Event )
			{
				// Overall progress info
				if( Event.State < EProgressionState.Idle )
				{
					switch( Event.State )
					{
					case EProgressionState.TaskTotal:
						NumTasks = Event.ThreadNum;
						break;

					case EProgressionState.TasksCompleted:
						NumRetiredTasks = Event.ThreadNum;
						break;

					case EProgressionState.TasksInProgress:
						NumRunningTasks = Event.ThreadNum;
						break;
					}

					return ( true );
				}

				// Handle per machine stats
				Progression MachineProgression = null;
				if( !MachineProgressions.TryGetValue( Event.Machine, out MachineProgression ) )
				{
					MachineProgressions.Add( Event.Machine, new Progression() );
					MachineProgressions.TryGetValue( Event.Machine, out MachineProgression );
				}

				List<ProgressionSample> Samples = MachineProgression.ProgressionSamples;
				ProgressionSample LastSample = Samples[Samples.Count - 1];

				// If this machine has disconnected, ignore any further messages
                if (LastSample.State == EProgressionState.RemoteDisconnected || LastSample.State == EProgressionState.InstigatorDisconnected)
				{
					return ( true );
				}

				// Coarse tracking on a per machine basis
				bool bCreateNewSample =
					(LastSample.State != Event.State) ||
					(Event.ThreadNum >= 0 && LastSample.ThreadSamples == null) ||
					(Event.ThreadNum < 0 && LastSample.ThreadSamples != null);

				if( bCreateNewSample == false )
				{
					// Just updating the same event, add timestamp
					LastSample.AddTimestamp( Event.Time );
				}
				else
				{
					// If the previous sample have proper Start and End timestamps, add one
					if( LastSample.NumTimestamps <= 1 )
					{
						LastSample.AddTimestamp( Event.Time );
					}
					// If the previous sample has individual thread samples, make sure they're all ended
					if( ( LastSample.ThreadSamples != null ) &&
						( LastSample.ThreadSamples.Count > 0 ) )
					{
						foreach( ProgressionThreadSample NextThreadSample in LastSample.ThreadSamples )
						{
							if( NextThreadSample.NumTimestamps <= 1 )
							{
								NextThreadSample.AddTimestamp( Event.Time );
							}
						}
					}
				}

				if( Event.ThreadNum < 0 )
				{
					if ( bCreateNewSample )
					{
						Samples.Add( new ProgressionSample( Event.State, Event.Time ) );
					}
				}
				else if( Event.ThreadNum >= 0 )
				{
					if ( bCreateNewSample )
					{
						// Previous sample was of a different kind, so create a new per-thread instance.
						LastSample = new ProgressionSample( Event.State, Event.Time );
						LastSample.ThreadSamples = new List<ProgressionThreadSample>();
						Samples.Add( LastSample );
					}

					// Optional more detailed per thread tracking
					while( LastSample.ThreadSamples.Count <= Event.ThreadNum )
					{
						List<ProgressionThreadSample> ThreadSamples = new List<ProgressionThreadSample>();
						ProgressionThreadSample ThreadSample = new ProgressionThreadSample( Event.State );
						LastSample.ThreadSamples.Add( ThreadSample );
					}

					LastSample.ThreadSamples[Event.ThreadNum].AddTimestamp( Event.Time );
				}

				// If we've had a disconnect message, make sure all progressions are terminated
				if( Event.State == EProgressionState.InstigatorDisconnected )
				{
					Samples.Add( new ProgressionSample( Event.State, Event.Time ) );

					// Make sure all progress bars stop as there will be no more messages
					foreach( Progression ProgressionMachine in MachineProgressions.Values )
					{
						List<ProgressionSample> ProgressionSamples = ProgressionMachine.ProgressionSamples;
						LastSample = ProgressionSamples[ProgressionSamples.Count - 1];

                        if (LastSample.State != EProgressionState.RemoteDisconnected && LastSample.State != EProgressionState.InstigatorDisconnected)
						{
							ProgressionSamples.Add( new ProgressionSample( EProgressionState.RemoteDisconnected, Event.Time ) );
						}
					}

					return ( true );
				}

				return ( bCreateNewSample );
			}

			/*
			 * Advance all active timers
			 */
			public bool Tick()
			{
				bool Invalidate = false;

				foreach( Progression MachineProgression in MachineProgressions.Values )
				{
					List<ProgressionSample> Samples = MachineProgression.ProgressionSamples;
					ProgressionSample Sample = Samples[Samples.Count - 1];
                    if (Sample.State != EProgressionState.RemoteDisconnected && Sample.State != EProgressionState.InstigatorDisconnected)
					{
						Sample.AddTimestamp( DateTime.UtcNow );
						Invalidate = true;
					}
				}

				return ( Invalidate );
			}
		}

		private bool VisualiserGridViewResized = false;

		/*
		 * Work out ideal height of progress bars
		 */
		private int CalculateIdealBarHeight( int NumMachines )
		{
			int TotalHeight = VisualiserGridView.Height - ( VisualiserGridView.ColumnHeadersHeight * 2 );
			int RowHeight = TotalHeight / ( NumMachines * 8 );

			if( RowHeight < 3 )
			{
				RowHeight = 3;
			}

			if( RowHeight > 12 )
			{
				RowHeight = 12;
			}

			return( RowHeight * 8 );
		}

		private static int CompareByStateAndTime( KeyValuePair<string, Progression> A, KeyValuePair<string, Progression> B )
		{
			int LastIndexA = A.Value.ProgressionSamples.Count - 1;
			int LastIndexB = B.Value.ProgressionSamples.Count - 1;
			// Start with state
			if( A.Value.ProgressionSamples[LastIndexA].State == EProgressionState.Blocked )
			{
				if( B.Value.ProgressionSamples[LastIndexB].State == EProgressionState.Blocked )
				{
					// Now sort on the time
					if( A.Value.ProgressionSamples[0].StartTime < B.Value.ProgressionSamples[0].StartTime )
					{
						return -1;
					}
					else if( A.Value.ProgressionSamples[0].StartTime == B.Value.ProgressionSamples[0].StartTime )
					{
						return 0;
					}
					else
					{
						return 1;
					}
				}
				else
				{
					return 1;
				}
			}
			else
			{
				if( B.Value.ProgressionSamples[LastIndexB].State == EProgressionState.Blocked )
				{
					return -1;
				}
				else
				{
					// Now sort on the time
					if( A.Value.ProgressionSamples[0].StartTime < B.Value.ProgressionSamples[0].StartTime )
					{
						return -1;
					}
					else if( A.Value.ProgressionSamples[0].StartTime == B.Value.ProgressionSamples[0].StartTime )
					{
						return 0;
					}
					else
					{
						return 1;
					}
				}
			}
		}
		
		/*
		 * Populate the gridview if the number of connected machines has changed
		 */
		public void PopulateGridView()
		{
			if( VisualiserGridViewResized || ProgressionData.MachineProgressions.Count != VisualiserGridView.Rows.Count )
			{
				VisualiserGridView.Rows.Clear();

				// Sort the machines by state and then by time
				List< KeyValuePair<string, Progression> > CopyOfProgressionData = 
					new List< KeyValuePair<string, Progression> >( ProgressionData.MachineProgressions.Copy() );
				
				// Disable the sort for now, because lots of movement is distracting
				//CopyOfProgressionData.Sort( CompareByStateAndTime );

				List<string> MachineNames = new List<string>();
				foreach( KeyValuePair<string, Progression> NextPair in CopyOfProgressionData )
				{
					MachineNames.Add( NextPair.Key );
				}
				
				// Always move the local machine name to the top of the list
				if( MachineNames.Remove(System.Net.Dns.GetHostName()) )
				{
					MachineNames.Insert( 0, System.Net.Dns.GetHostName());
				}

				// Find ideal row height dependent on number of agents
				int RowHeight = CalculateIdealBarHeight( MachineNames.Count );

				foreach( string Machine in MachineNames )
				{
					DataGridViewRow Row = new DataGridViewRow();
					Row.Height = RowHeight;

					DataGridViewTextBoxCell NameCell = new DataGridViewTextBoxCell();
					NameCell.Value = Machine;
					Row.Cells.Add( NameCell );

					DataGridViewTextBoxCell ProgressCell = new DataGridViewTextBoxCell();
					Row.Cells.Add( ProgressCell );

					VisualiserGridView.Rows.Add( Row );
				}

				VisualiserGridViewResized = false;
				VisualiserGridView.Invalidate();
			}

			// Handle the horizontal scroll bar
			VisualiserGridView.Columns[1].AutoSizeMode = DataGridViewAutoSizeColumnMode.None;

			// Get the largest width of each row
			int ProgressionWidth = 0;
			foreach( Progression ProgressionMachine in ProgressionData.MachineProgressions.Values )
			{
				if ( ProgressionMachine.ProgressTotalWidth > ProgressionWidth )
				{
					ProgressionWidth = ProgressionMachine.ProgressTotalWidth;
				}
			}
			ProgressionWidth += 10;	// Add a small margin.

			// Figure out the desired width of the 2nd column.
			int ColumnWidth = VisualiserGridView.Columns[1].Width;
			if ( ColumnWidth <= ProgressionWidth || ColumnWidth > (ProgressionWidth + 200) )
			{
				ColumnWidth = ProgressionWidth + 200;
			}

			// Make sure it never goes smaller than the container
			int ContainerWidth = VisualiserGridView.Width - VisualiserGridView.Columns[0].Width - 3;
			if( ColumnWidth < ContainerWidth )
			{
				ColumnWidth = ContainerWidth;
			}

			// WINDOWS BUG: no width larger than 64k
			if( ColumnWidth > 65536 )
			{
				ColumnWidth = 65536;
			}

			if( VisualiserGridView.Columns[1].Width != ColumnWidth )
			{
				VisualiserGridView.Columns[1].Width = ColumnWidth;
				VisualiserGridView.Invalidate();
			}
		}

		/*
		 * Draw a scale in a column header
		 */
		private const int TickBorder = 4;

		private void DrawScale( DateTime StartTime, float ZoomLevel, Graphics Gfx, Rectangle Bounds )
		{
			int Offset = 0;
			int Count = 1;
			while( Offset < Bounds.Width )
			{
				Offset = ( int )( 15.0f * ZoomLevel * Count );

				int Left = Bounds.Left + Offset;
				int Width = 1;
				int Top = Bounds.Top + TickBorder;
				int Height = Bounds.Height - ( TickBorder * 2 );

				if( ( Count & 3 ) != 0 )
				{
					Top += TickBorder;
					Height -= TickBorder * 2;
				}

				Rectangle Bar = new Rectangle( Left, Top, Width, Height );
				Gfx.DrawRectangle( Pens.Black, Bar );

				Count++;
			}
		}

		/*
		 * Paint a cell using the progression data
		 */
		private void ProgressCellPaint( object sender, DataGridViewCellPaintingEventArgs e )
		{
			if( e.ColumnIndex <= 0 || e.RowIndex < -1 )
			{
				e.Handled = false;
				return;
			}

			if( ProgressionData != null )
			{
				if( e.RowIndex == -1 )
				{
					e.PaintBackground( e.CellBounds, false );
					DrawScale( ProgressionData.StartTime, AgentApplication.Options.VisualiserZoomLevel, e.Graphics, e.CellBounds );
				}
				else
				{
					string Machine = ( string )VisualiserGridView.Rows[e.RowIndex].Cells[0].Value;
					Progression MachineProgression = null;
					if( ProgressionData.MachineProgressions.TryGetValue( Machine, out MachineProgression ) )
					{
						BufferedGraphicsContext Current = BufferedGraphicsManager.Current;
						BufferedGraphics Offscreen = Current.Allocate( e.Graphics, e.CellBounds );

						// Fill the background dependent on whether the machine has finished or not
						MachineProgression.DrawBackground( Offscreen.Graphics, e.CellBounds );

						// Draw all the progress bars to the offscreen buffer
						MachineProgression.Draw( ProgressionData.StartTime, AgentApplication.Options.VisualiserZoomLevel, Offscreen.Graphics, e.CellBounds );

						// Copy the offscreen buffer to the screen
						Offscreen.Render();
						Offscreen.Dispose();
					}
				}
			}
			else
			{
				// Just clear if there's no progression data
				e.PaintBackground( e.CellBounds, false );
			}

			e.Handled = true;
		}

		private void MouseWheelHandler( object sender, MouseEventArgs e )
		{
			float ZoomLevel = AgentApplication.Options.VisualiserZoomLevel + ( e.Delta / 120.0f );

			if( ZoomLevel < 1.0f )
			{
				ZoomLevel = 1.0f;
			}
			else if( ZoomLevel > 100.0f )
			{
				ZoomLevel = 100.0f;
			}

			AgentApplication.Options.VisualiserZoomLevel = ZoomLevel;
			VisualiserGridView.Invalidate();
		}

		private void MouseMoveHandler(object sender, MouseEventArgs e)
		{
			Point GridClientPos = this.VisualiserGridView.PointToClient( MousePosition );
			DataGridView.HitTestInfo HitInfo = this.VisualiserGridView.HitTest( GridClientPos.X, GridClientPos.Y );
			bool bShowingToolTip = false;
			if ( HitInfo.Type == DataGridViewHitTestType.ColumnHeader && HitInfo.ColumnIndex == 1 )
			{
				float Timestamp = GridClientPos.X - HitInfo.ColumnX;
				Timestamp /= AgentApplication.Options.VisualiserZoomLevel;
				Point AgentClientPos = PointToClient( MousePosition );
				string ToolTipText = String.Format( "Time: {0:g4} seconds", Timestamp );
				BarToolTip.Show(ToolTipText, this, AgentClientPos.X, AgentClientPos.Y, 60000 );
				bShowingToolTip = true;
			}
			else if ( HitInfo.Type == DataGridViewHitTestType.Cell && HitInfo.ColumnIndex == 1 )
			{
				DataGridViewCell cell = this.VisualiserGridView.Rows[HitInfo.RowIndex].Cells[1];
				string Machine = ( string )VisualiserGridView.Rows[HitInfo.RowIndex].Cells[0].Value;
				Progression MachineProgression = null;
				if ( ProgressionData.MachineProgressions.TryGetValue( Machine, out MachineProgression ) )
				{
					for( int BarIndex = 1; BarIndex < MachineProgression.ProgressionSamples.Count; BarIndex++ )
					{
						ProgressionSample Sample = MachineProgression.ProgressionSamples[BarIndex];
						if ( Sample.Bounds.Contains(GridClientPos) )
						{
							Point AgentClientPos = PointToClient( MousePosition );
							string BarName = BarNames[Sample.State.ToString()];
							string ToolTipText = String.Format( "{0} ({1:g4} seconds)", BarName, Sample.Duration );
							BarToolTip.Show(ToolTipText, this, AgentClientPos.X, AgentClientPos.Y, 60000 );
							bShowingToolTip = true;
							break;
						}
					}
				}
			}

			if ( !bShowingToolTip )
			{
				BarToolTip.Hide( this );
			}
		}

		private void MouseLeaveHandler(object sender, EventArgs e)
		{
			BarToolTip.Hide( this );
		}

		private void GridViewSelectionChanged( object sender, EventArgs e )
		{
			VisualiserGridView.ClearSelection();
		}

		private void GridViewResized( object sender, EventArgs e )
		{
			VisualiserGridViewResized = true;
		}

		private void OverallProgressBarPaint( object sender, PaintEventArgs e )
		{
			if( ProgressionData != null )
			{
				BufferedGraphicsContext Current = BufferedGraphicsManager.Current;
				BufferedGraphics Offscreen = Current.Allocate( e.Graphics, OverallProgressBar.ClientRectangle );

				ProgressionData.DrawOverallProgressBar( Offscreen.Graphics, OverallProgressBar.ClientRectangle );

				// Copy the offscreen buffer to the screen
				Offscreen.Render();
				Offscreen.Dispose();
			}
		}
	}
}

