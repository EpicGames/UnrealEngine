// Copyright Epic Games, Inc. All Rights Reserved.


using System.Reflection;

// General Information about an assembly is controlled through the following 
// set of attributes. Change these attribute values to modify the information
// associated with an assembly.
[assembly: AssemblyTitle( "SwarmAgent" )]
[assembly: AssemblyDescription("A tool to allow distribution of tasks")]
[assembly: AssemblyConfiguration("")]

// Version information for an assembly consists of the following four values:
//
//      Major Version
//      Minor Version 
//      Build Number
//      Revision
//
[assembly: AssemblyVersion( "1.7.2893.0" )]
