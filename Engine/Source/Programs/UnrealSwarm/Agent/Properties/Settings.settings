<?xml version='1.0' encoding='utf-8'?>
<SettingsFile xmlns="http://schemas.microsoft.com/VisualStudio/2004/01/settings" CurrentProfile="(Default)" GeneratedClassNamespace="Agent.Properties" GeneratedClassName="Settings">
  <Profiles />
  <Settings>
    <Setting Name="CoordinatorRemotingPort" Type="System.Int32" Scope="Application">
      <Value Profile="(Default)">8009</Value>
    </Setting>
    <Setting Name="AgentRemotingPort" Type="System.Int32" Scope="Application">
      <Value Profile="(Default)">8008</Value>
    </Setting>
    <Setting Name="StatsDatabaseName" Type="System.String" Scope="Application">
      <Value Profile="(Default)">engine-db</Value>
    </Setting>
  </Settings>
</SettingsFile>