// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.ComponentModel;
#if !__MonoCS__
using System.Deployment.Application;
#endif
using System.Diagnostics;
using System.Drawing;
using System.Drawing.Design;
using System.IO;
using System.Reflection;
using System.Xml;
using System.Xml.Serialization;

using AgentInterface;

namespace Agent
{
	public enum EOptionsVersion
	{
		OriginalVersion						= 3,
		NewBarColors						= 4,
		CacheManagement						= 5,
		NewStartupColor						= 6,
		AdjustedProgressionColors			= 7,
		TweaksToTheNumberOfThreads			= 8,
		FixTheOptionsReset					= 9,
		DeployNewFarmMachineNames			= 10,
		BumpTheJobsToKeepCount				= 11,
		MoveTheDefaultCacheFolder			= 12,
		ChangeCoordinatorLocation			= 13,
		AddPerformanceMonitoringDisable		= 14,
		MakeLocalJobsProcessorsCountLimited = 15,

		Current								= MakeLocalJobsProcessorsCountLimited
	};

	public class EnumedCollectionPropertyDescriptor<E, T> : PropertyDescriptor
	{
		private EnumedCollection<E, T> Collection = null;
		private int Index = -1;

		public EnumedCollectionPropertyDescriptor( EnumedCollection<E, T> InCollection, int InIndex ) :
			base( "#" + InIndex.ToString(), null )
		{
			Collection = InCollection;
			Index = InIndex;
		}

		public override AttributeCollection Attributes
		{
			get { return( new AttributeCollection( null ) ); }
		}

		public override bool CanResetValue( object component )
		{
			return( true );
		}

		public override Type ComponentType
		{
			get { return( Collection.GetType() ); }
		}

		public override string Description
		{
			get { return ( "The colour representing this state of the job." ); }
		}

		public override string DisplayName
		{
			get { return( Enum.GetName( typeof( E ), Index ) ); }
		}

		public override object GetValue( object component )
		{
			return( Collection[Index] );
		}

		public override bool IsReadOnly
		{
			get { return( false ); }
		}

		public override string Name
		{
			get { return( "#" + Index.ToString() ); }
		}

		public override Type PropertyType
		{
			get { return( typeof( T ) ); }
		}

		public override void ResetValue( object component )
		{
		}

		public override bool ShouldSerializeValue( object component )
		{
			return( true );
		}

		public override void SetValue( object component, object value )
		{
			Collection[Index] = ( T )value;
		}
	}

	public class EnumedCollection<E, T> : CollectionBase, ICustomTypeDescriptor
	{
		public void Add( T BarColour )
		{
			List.Add( BarColour );
		}

		public void Remove( T BarColour )
		{
			List.Remove( BarColour );
		}

		public T this[int Index]
		{
			get { return ( ( T )List[Index] ); }
			set { List[Index] = ( T )value; }
		}

		// Implementation of interface ICustomTypeDescriptor 
		public String GetClassName()
		{
			return TypeDescriptor.GetClassName( this, true );
		}

		public AttributeCollection GetAttributes()
		{
			return TypeDescriptor.GetAttributes( this, true );
		}

		public String GetComponentName()
		{
			return TypeDescriptor.GetComponentName( this, true );
		}

		public TypeConverter GetConverter()
		{
			return TypeDescriptor.GetConverter( this, true );
		}

		public EventDescriptor GetDefaultEvent()
		{
			return TypeDescriptor.GetDefaultEvent( this, true );
		}

		public PropertyDescriptor GetDefaultProperty()
		{
			return TypeDescriptor.GetDefaultProperty( this, true );
		}

		public object GetEditor( Type editorBaseType )
		{
			return TypeDescriptor.GetEditor( this, editorBaseType, true );
		}

		public EventDescriptorCollection GetEvents( Attribute[] attributes )
		{
			return TypeDescriptor.GetEvents( this, attributes, true );
		}

		public EventDescriptorCollection GetEvents()
		{
			return TypeDescriptor.GetEvents( this, true );
		}

		public object GetPropertyOwner( PropertyDescriptor pd )
		{
			return this;
		}

		public PropertyDescriptorCollection GetProperties( Attribute[] attributes )
		{
			return GetProperties();
		}

		public PropertyDescriptorCollection GetProperties()
		{
			PropertyDescriptorCollection PropertyDescriptions = new PropertyDescriptorCollection( null );

			for( int i = 0; i < List.Count; i++ )
			{
				EnumedCollectionPropertyDescriptor<E, T> PropertyDescriptor = new EnumedCollectionPropertyDescriptor<E, T>( this, i );
				PropertyDescriptions.Add( PropertyDescriptor );
			}

			return( PropertyDescriptions );
		}
	}

	internal class EnumedCollectionConverter : ExpandableObjectConverter
	{
		public override object ConvertTo( ITypeDescriptorContext Context, System.Globalization.CultureInfo Culture, object Value, Type DestType )
		{
			if( DestType == typeof( string ) && ( Value is EnumedCollection<EProgressionState, Color> ) )
			{
				return( "Progress bar colors" );
			}

			return base.ConvertTo( Context, Culture, Value, DestType );
		}
	}

	public class SuppressCollectionEditor : UITypeEditor
	{
		public override UITypeEditorEditStyle GetEditStyle( ITypeDescriptorContext context )
		{
			return( UITypeEditorEditStyle.None );
		}
	}

	public class SerialisableColour
	{
		public SerialisableColour()
		{
			Colour = Color.Black;
		}

		public SerialisableColour( Color InColour )
		{
			Colour = InColour;
		}

		[XmlIgnore]
		public Color Colour
		{
			get { return ( Color.FromArgb( LocalColour ) ); }
			set { LocalColour = value.ToArgb(); }
		}

		[XmlAttribute]
		public int LocalColour = 0;
	}

    public class SettableOptions
    {
        [CategoryAttribute( "Log Settings" )]
        [DescriptionAttribute( "The amount of text spewed to the window." )]
		[DefaultValueAttribute( EVerbosityLevel.Informative )]
        public EVerbosityLevel Verbosity
        {
            get { return ( LocalVerbosity ); }
            set { LocalVerbosity = value; }
        }
        [XmlEnumAttribute]
        private EVerbosityLevel LocalVerbosity = EVerbosityLevel.Informative;

		[CategoryAttribute( "Log Settings" )]
		[DescriptionAttribute( "The font used in all Swarm Agent dialogs." )]
		[DefaultValueAttribute( SwarmAgentWindow.DialogFont.Tahoma )]
		public SwarmAgentWindow.DialogFont TextFont
		{
			get { return ( LocalTextFont ); }
			set { LocalTextFont = value; }
		}
		[XmlAttribute]
		private SwarmAgentWindow.DialogFont LocalTextFont = SwarmAgentWindow.DialogFont.Tahoma;

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "Whether to allow only distribution of jobs and tasks from this agent (no local execution)." )]
		[DefaultValueAttribute( false )]
		public bool AvoidLocalExecution
		{
			get { return ( LocalAvoidLocalExecution ); }
			set { LocalAvoidLocalExecution = value; }
		}
		[XmlAttribute]
		private bool LocalAvoidLocalExecution = false;

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "Enables Standalone mode, which disables the distribution system for both outgoing and incoming tasks." )]
		[DefaultValueAttribute( false )]
		public bool EnableStandaloneMode
		{
			get { return ( LocalEnableStandaloneMode ); }
			set { LocalEnableStandaloneMode = value; }
		}
		[XmlAttribute]
		private bool LocalEnableStandaloneMode = false;

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "[Advanced] The remote machine filter string (' ', ',' or ';' delimited)." )]
		[DefaultValueAttribute( "RENDER*" )]
		public string AllowedRemoteAgentNames
		{
			get { return ( LocalAllowedRemoteAgentNames ); }
			set { LocalAllowedRemoteAgentNames = value; }
		}
		[XmlTextAttribute]
		private string LocalAllowedRemoteAgentNames = "RENDER*";

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "[Advanced] The name of the agent group jobs can be distributed to." )]
		[DefaultValueAttribute( "DefaultDeployed" )]
		public string AllowedRemoteAgentGroup
		{
			get { return ( LocalAllowedRemoteAgentGroup ); }
			set { LocalAllowedRemoteAgentGroup = value; }
		}
		[XmlTextAttribute]
		private string LocalAllowedRemoteAgentGroup = "DefaultDeployed";

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "[Advanced] The name of the agent group to which this machine belongs." )]
		[DefaultValueAttribute( "Default" )]
		public string AgentGroupName
		{
			get { return ( LocalAgentGroupName ); }
			set { LocalAgentGroupName = value; }
		}
		[XmlTextAttribute]
#if !__MonoCS__
		private string LocalAgentGroupName = ( ApplicationDeployment.IsNetworkDeployed ? "DefaultDeployed" : "Default" );
#else
		private string LocalAgentGroupName = "Default";
#endif

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "[Advanced] The name of the machine hosting the Coordinator." )]
		[DefaultValueAttribute( "RENDER-01" )]
		public string CoordinatorRemotingHost
		{
			get { return ( LocalCoordinatorRemotingHost ); }
			set { LocalCoordinatorRemotingHost = value; }
		}
		[XmlTextAttribute]
		private string LocalCoordinatorRemotingHost = "RENDER-01";

		[CategoryAttribute( "Cache Settings" )]
        [DescriptionAttribute( "The location of the cache folder. This should be on a fast drive with plenty of free space." )]
		[DefaultValueAttribute( "C:\\SwarmCache" )]
		public string CacheFolder
        {
            get { return ( LocalCacheFolder ); }
            set { LocalCacheFolder = value; }
        }
		[XmlTextAttribute]
		private string LocalCacheFolder = "C:\\SwarmCache";

		[CategoryAttribute( "Cache Settings" )]
		[DescriptionAttribute( "The number of previous jobs to keep the logs and output data for." )]
		[DefaultValueAttribute( 10 )]
		public Int32 MaximumJobsToKeep
		{
			get { return ( LocalMaximumJobsToKeep ); }
			set { LocalMaximumJobsToKeep = value; }
		}
		[XmlTextAttribute]
		private Int32 LocalMaximumJobsToKeep = 10;

		[CategoryAttribute( "Cache Settings" )]
		[DescriptionAttribute( "The approximate maximum size, in gigabytes, of the cache folder." )]
		[DefaultValueAttribute( 10 )]
		public Int32 MaximumCacheSize
		{
			get { return ( LocalMaximumCacheSize ); }
			set { LocalMaximumCacheSize = value; }
		}
		[XmlTextAttribute]
		private Int32 LocalMaximumCacheSize = 10;

		[CategoryAttribute( "General Settings" )]
		[DescriptionAttribute( "If enabled, brings the Agent window to the front when a Job is started." )]
		[DefaultValueAttribute( true )]
		public bool BringToFront
		{
			get { return ( LocalBringToFront ); }
			set { LocalBringToFront = value; }
		}
		[XmlEnumAttribute]
		private bool LocalBringToFront = false;

		[CategoryAttribute( "Visualizer Settings" )]
		[DescriptionAttribute( "The colors used to draw the progress bars." )]
		[Editor( typeof( SuppressCollectionEditor ), typeof( UITypeEditor ) )]
		[TypeConverter( typeof( EnumedCollectionConverter ) )]
		[XmlIgnore]
		public EnumedCollection<EProgressionState, Color> VisualizerColors
		{
			get { return LocalColours; }
		}
		private EnumedCollection<EProgressionState, Color> LocalColours = new EnumedCollection<EProgressionState, Color>();

		[BrowsableAttribute( false )]
		public List<SerialisableColour> SerialisableColours = new List<SerialisableColour>();

		[CategoryAttribute( "Developer Settings" )]
		[DescriptionAttribute( "[DEVELOPER AND QA USE ONLY] If enabled, shows the developer menu." )]
		[DefaultValueAttribute( false )]
		public bool ShowDeveloperMenu
		{
			get { return ( LocalShowDeveloperMenu ); }
			set { LocalShowDeveloperMenu = value; }
		}
		[XmlEnumAttribute]
		private bool LocalShowDeveloperMenu = false;

		/*
		 * State variables that do not appear in the property grid, but are still serialized out
		 */
		[BrowsableAttribute( false )]
		public int OptionsVersion
		{
			get { return ( LocalOptionsVersion ); }
			set { LocalOptionsVersion = value; }
		}
		[XmlAttribute]
		private int LocalOptionsVersion = (int)EOptionsVersion.Current;

		[BrowsableAttribute( false )]
		public Point WindowLocation
		{
			get { return ( LocalWindowLocation ); }
			set { LocalWindowLocation = value; }
		}
		[XmlAttribute]
		private Point LocalWindowLocation = new Point( 0, 0 );

		[BrowsableAttribute( false )]
		public Size WindowSize
		{
			get { return ( LocalWindowSize ); }
			set { LocalWindowSize = value; }
		}
		[XmlAttribute]
		private Size LocalWindowSize = new Size( 768, 768 );

		[BrowsableAttribute( false )]
		public int AgentTabIndex
		{
			get { return ( LocalAgentTabIndex ); }
			set { LocalAgentTabIndex = value; }
		}
		[XmlAttribute]
		private int LocalAgentTabIndex = 0;

		[BrowsableAttribute( false )]
		public float VisualiserZoomLevel
		{
			get { return ( LocalVisualiserZoomLevel ); }
			set { LocalVisualiserZoomLevel = value; }
		}
		[XmlAttribute]
		private float LocalVisualiserZoomLevel = 4.0f;

		/*
		 * Set up any required default states
		 */
		private void SetDefaults()
		{
			Verbosity = EVerbosityLevel.Informative;
			TextFont = SwarmAgentWindow.DialogFont.Tahoma;
			AvoidLocalExecution = false;
			EnableStandaloneMode = false;
			AllowedRemoteAgentNames = "RENDER*";
			AllowedRemoteAgentGroup = "DefaultDeployed";
#if !__MonoCS__
			AgentGroupName = ( ApplicationDeployment.IsNetworkDeployed ? "DefaultDeployed" : "Default" );
#else
			AgentGroupName = "Default";
#endif
#if !__MonoCS__
			// The cache folder location depends on how the Agent is deployed
			if( ApplicationDeployment.IsNetworkDeployed )
			{
				// If we're network deployed, we can't make assumptions about the drives available,
				// so stick with a reasonable default
				string LocationRoot = Path.GetPathRoot( Assembly.GetExecutingAssembly().Location );
				if( LocationRoot != null )
				{
					// Base it at the root of the drive this executable is running from
					CacheFolder = Path.Combine( LocationRoot, "SwarmCache" );
				}
				else
				{
					// The old stand-by...
					CacheFolder = "C:\\SwarmCache";
				}
			}
			else
#endif
            if ( AgentApplication.OptionsFolder.Length > 0 )
            {
                // If an options folder was specified, set the default CacheFolder to be within it
                CacheFolder = Path.Combine(AgentApplication.OptionsFolder, "SwarmCache");
            }
            else
			{
				// Otherwise, keep the cache close to the Agent using it so that it's easy to
				// locate, easy to end up on the faster drive in the system, and easier to
				// delete when removing the Agent
				CacheFolder = Path.GetDirectoryName( Assembly.GetExecutingAssembly().Location ) + "/SwarmCache";
			}
			MaximumJobsToKeep = 5;
			MaximumCacheSize = 10;
			BringToFront = true;
			ShowDeveloperMenu = false;
	
			SerialisableColours.Clear();

			SerialisableColours.Add( new SerialisableColour( Color.White ) );				// PROGSTATE_TaskTotal					
			SerialisableColours.Add( new SerialisableColour( Color.White ) );				// PROGSTATE_TasksInProgress				
			SerialisableColours.Add( new SerialisableColour( Color.White ) );				// PROGSTATE_TasksCompleted					
			SerialisableColours.Add( new SerialisableColour( Color.White ) );				// PROGSTATE_Idle					
			SerialisableColours.Add( new SerialisableColour( Color.DarkGray ) );			// PROGSTATE_InstigatorConnected,
			SerialisableColours.Add( new SerialisableColour( Color.DarkGray ) );			// PROGSTATE_RemoteConnected,
			SerialisableColours.Add( new SerialisableColour( Color.DarkGray ) );			// PROGSTATE_Exporting,
			SerialisableColours.Add( new SerialisableColour( Color.LightGray ) );			// PROGSTATE_BeginJob,
			SerialisableColours.Add( new SerialisableColour( Color.White ) );				// PROGSTATE_Blocked,
			SerialisableColours.Add( new SerialisableColour( Color.DarkRed ) );				// PROGSTATE_Preparing0,
			SerialisableColours.Add( new SerialisableColour( Color.DarkRed ) );				// PROGSTATE_Preparing1,
			SerialisableColours.Add( new SerialisableColour( Color.Chocolate ) );			// PROGSTATE_Preparing2,
			SerialisableColours.Add( new SerialisableColour( Color.Chocolate ) );			// PROGSTATE_Preparing3,
			SerialisableColours.Add( new SerialisableColour( Color.DarkBlue ) );			// PROGSTATE_Processing0,
			SerialisableColours.Add( new SerialisableColour( Color.DarkBlue ) );			// PROGSTATE_Processing1,
			SerialisableColours.Add( new SerialisableColour( Color.DarkBlue ) );			// PROGSTATE_Processing2,
			SerialisableColours.Add( new SerialisableColour( Color.DarkBlue ) );			// PROGSTATE_Processing3,
			SerialisableColours.Add( new SerialisableColour( Color.DarkBlue ) );			// PROGSTATE_FinishedProcessing0,
			SerialisableColours.Add( new SerialisableColour( Color.DarkBlue ) );			// PROGSTATE_FinishedProcessing1,
			SerialisableColours.Add( new SerialisableColour( Color.LightGreen ) );			// PROGSTATE_FinishedProcessing2,
			SerialisableColours.Add( new SerialisableColour( Color.Green ) );				// PROGSTATE_FinishedProcessing3,
			SerialisableColours.Add( new SerialisableColour( Color.Cyan ) );				// PROGSTATE_ExportingResults,
			SerialisableColours.Add( new SerialisableColour( Color.Purple ) );				// PROGSTATE_ImportingResults,
			SerialisableColours.Add( new SerialisableColour( Color.Green ) );				// PROGSTATE_Finished,
			SerialisableColours.Add( new SerialisableColour( Color.Black ) );				// PROGSTATE_RemoteDisconnected
			SerialisableColours.Add( new SerialisableColour( Color.Black ) );				// PROGSTATE_InstigatorDisconnected
            SerialisableColours.Add( new SerialisableColour( Color.Black));			        // PROGSTATE_Preparing4

			OptionsVersion = (int)EOptionsVersion.Current;
		}

		/*
		 * Validate the collections
		 */
		private void Validate()
		{
			if( OptionsVersion < (int)EOptionsVersion.Current || SerialisableColours.Count == 0 )
			{
				SetDefaults();
			}

			// Ensure we have enough colours for the enum
			for( int Index = SerialisableColours.Count; Index < ( int )EProgressionState.Num; Index++ )
			{
				SerialisableColours.Add( new SerialisableColour( Color.Black ) );
			}
		}

		/*
		 * Copy the property grid values to a serializable version
		 */
		public void PreSave()
		{
			// Copy the property grid friendly data to the xml friendly data
			// (should not be required)
			SerialisableColours.Clear();

			foreach( Color Colour in VisualizerColors )
			{
				SerialisableColours.Add( new SerialisableColour( Colour ) );
			}
		}

		/*
		 * Fix up anything that can't the automatically serialized
		 */
		public void PostLoad()
		{
			// Make sure the loaded data is valid
			Validate();

			// Copy the xml friendly data to the property grid friendly data
			// (should not be required)
			VisualizerColors.Clear(); 

			foreach( SerialisableColour Colour in SerialisableColours )
			{
				VisualizerColors.Add( Colour.Colour );
			}
		}
	}

	// A limited range of values for the process priority enum
	public enum AgentProcessPriorityClass
	{
		Normal = ProcessPriorityClass.Normal,
		Idle = ProcessPriorityClass.Idle,
		BelowNormal = ProcessPriorityClass.BelowNormal,
		AboveNormal = ProcessPriorityClass.AboveNormal
	}

	public class SettableDeveloperOptions
	{
		[CategoryAttribute( "Developer Settings" )]
		[DescriptionAttribute( "[DEVELOPER AND QA USE ONLY] Enables a distribution mode which attempts to reproduce the distribution behavior of the last Job." )]
		[DefaultValueAttribute( false )]
		public bool ReplayLastDistribution
		{
			get { return ( LocalReplayLastDistribution ); }
			set { LocalReplayLastDistribution = value; }
		}
		[XmlAttribute]
		private bool LocalReplayLastDistribution = false;

        [CategoryAttribute("Local Performance Settings")]
        [DescriptionAttribute("Whether to enable local performance monitoring")]
        public bool LocalEnableLocalPerformanceMonitoring
        {
            get { return (PrivateLocalEnableLocalPerformanceMonitoring); }
            set { PrivateLocalEnableLocalPerformanceMonitoring = value; }
        }
        [XmlAttribute]
        private bool PrivateLocalEnableLocalPerformanceMonitoring = true;

        [CategoryAttribute("Local Performance Settings")]
		[DescriptionAttribute( "The number of processors to make available to locally initiated jobs" )]
		public int LocalJobsDefaultProcessorCount
		{
			get { return ( PrivateLocalJobsDefaultProcessorCount ); }
			set { PrivateLocalJobsDefaultProcessorCount = value; }
		}
		[XmlAttribute]
		private int PrivateLocalJobsDefaultProcessorCount = GetDefaultLocalJobsDefaultProcessorCount();

		[CategoryAttribute( "Local Performance Settings" )]
		[DescriptionAttribute( "The default priority for locally initiated jobs" )]
		public AgentProcessPriorityClass LocalJobsDefaultProcessPriority
		{
			get { return ( PrivateLocalJobsDefaultProcessPriority ); }
			set { PrivateLocalJobsDefaultProcessPriority = value; }
		}
		[XmlAttribute]
		private AgentProcessPriorityClass PrivateLocalJobsDefaultProcessPriority = AgentProcessPriorityClass.BelowNormal;

		[CategoryAttribute( "Local Performance Settings" )]
		[DescriptionAttribute( "The number of processors to make available to remotely initiated jobs" )]
		public int RemoteJobsDefaultProcessorCount
		{
			get { return ( PrivateRemoteJobsDefaultProcessorCount ); }
			set { PrivateRemoteJobsDefaultProcessorCount = value; }
		}
		[XmlAttribute]
		private int PrivateRemoteJobsDefaultProcessorCount = GetDefaultRemoteJobsDefaultProcessorCount();

		[CategoryAttribute( "Local Performance Settings" )]
		[DescriptionAttribute( "The default priority for remotely initiated jobs" )]
		public AgentProcessPriorityClass RemoteJobsDefaultProcessPriority
		{
			get { return ( PrivateRemoteJobsDefaultProcessPriority ); }
			set { PrivateRemoteJobsDefaultProcessPriority = value; }
		}
		[XmlAttribute]
		private AgentProcessPriorityClass PrivateRemoteJobsDefaultProcessPriority = AgentProcessPriorityClass.Idle;

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "The amount of time to wait before forefully killing a Job executable after the Job is closed (in seconds). Set to anything < 0 for an infinite timeout." )]
		[DefaultValueAttribute( 5 )]
		public int JobExecutableTimeout
		{
			get { return ( LocalJobExecutableTimeout ); }
			set { LocalJobExecutableTimeout = value; }
		}
		[XmlAttribute]
		private int LocalJobExecutableTimeout = 5;


		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "The amount of time to wait before considering a remote Agent dropped (in seconds). Set to anything < 0 for an infinite timeout." )]
		[DefaultValueAttribute( 30 )]
		public int RemoteAgentTimeout
		{
			get { return ( LocalRemoteAgentTimeout ); }
			set { LocalRemoteAgentTimeout = value; }
		}
		[XmlAttribute]
		private int LocalRemoteAgentTimeout = 30;

		[CategoryAttribute( "Distribution Settings" )]
		[DescriptionAttribute( "[Advanced] Enables auto updating for deployed agents." )]
		[DefaultValueAttribute( true )]
		public bool UpdateAutomatically
		{
			get { return ( LocalUpdateAutomatically ); }
			set { LocalUpdateAutomatically = value; }
		}
		[XmlTextAttribute]
#if !__MonoCS__
		private bool LocalUpdateAutomatically = ( ApplicationDeployment.IsNetworkDeployed ? true : false );
#else
		private bool LocalUpdateAutomatically = false;
#endif

		[CategoryAttribute( "Log Settings" )]
		[DescriptionAttribute( "The maximum number of lines of output from a job application before it truncates what goes to the window. All of the output is written to the log on disk." )]
		[DefaultValueAttribute( 1000 )]
		public Int32 MaximumJobApplicationLogLines
		{
			get { return ( LocalMaximumJobApplicationLogLines ); }
			set { LocalMaximumJobApplicationLogLines = value; }
		}
		[XmlAttribute]
		private Int32 LocalMaximumJobApplicationLogLines = 1000;

		/*
		 * State variables that do not appear in the property grid, but are still serialized out
		 */
		[BrowsableAttribute( false )]
		public int OptionsVersion
		{
			get { return ( LocalOptionsVersion ); }
			set { LocalOptionsVersion = value; }
		}
		[XmlAttribute]
		private int LocalOptionsVersion = ( int )EOptionsVersion.Current;

		/**
		 * Gets allowed processors count on this machine for local tasks.
		 *
		 * This value can be lass than logical processors count as we don't
		 * want to block the editor on user's machine.
		 *
		 * @returns Allowed processors count.
		 */
		private static Int32 GetAllowedProcessorCount()
		{
			const Int32 NumUnusedSwarmThreads = 2;
			
			Int32 NumVirtualCores = Environment.ProcessorCount;
			Int32 NumThreads = NumVirtualCores - NumUnusedSwarmThreads;

			// On machines with few cores, each core will have a massive impact on build times, so we prioritize build latency over editor performance during the build
			if (NumVirtualCores <= 4)
			{
				NumThreads = NumVirtualCores - 1;
			}

			return Math.Max(1, NumThreads);
		}

		private static Int32 GetDefaultLocalJobsDefaultProcessorCount()
		{
			return
#if !__MonoCS__
				ApplicationDeployment.IsNetworkDeployed ? Environment.ProcessorCount : GetAllowedProcessorCount();
#else
				GetAllowedProcessorCount();
#endif
		}

		private static Int32 GetDefaultRemoteJobsDefaultProcessorCount()
		{
			return Environment.ProcessorCount;
		}

		/*
		 * Set up any required default states
		 */
		private void SetDefaults()
		{
			ReplayLastDistribution = false;
            LocalEnableLocalPerformanceMonitoring = true;

			LocalJobsDefaultProcessorCount = GetDefaultLocalJobsDefaultProcessorCount();
			LocalJobsDefaultProcessPriority = AgentProcessPriorityClass.BelowNormal;

			RemoteJobsDefaultProcessorCount = GetDefaultRemoteJobsDefaultProcessorCount();
			RemoteJobsDefaultProcessPriority = AgentProcessPriorityClass.Idle;

			JobExecutableTimeout = 5;
			RemoteAgentTimeout = 30;
#if !__MonoCS__
			UpdateAutomatically = ( ApplicationDeployment.IsNetworkDeployed ? true : false );
#else
			UpdateAutomatically = false;
#endif
			MaximumJobApplicationLogLines = 1000;

			OptionsVersion = ( int )EOptionsVersion.Current;
		}

		/*
		 * Validate the collections
		 */
		private void Validate()
		{
			if( OptionsVersion < ( int )EOptionsVersion.Current )
			{
				SetDefaults();
			}
		}

		/*
		 * Copy the property grid values to a serializable version
		 */
		public void PreSave()
		{
		}

		/*
		 * Fix up anything that can't the automatically serialized
		 */
		public void PostLoad()
		{
			// Make sure the loaded data is valid
			Validate();
		}
	}

}
