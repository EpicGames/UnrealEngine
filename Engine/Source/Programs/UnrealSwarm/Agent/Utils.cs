// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Management;
using System.Threading;
using System.Windows;

using AgentInterface;

namespace Agent
{
	///////////////////////////////////////////////////////////////////////////

	/**
	 * A thread-safe reader-writer-locked Dictionary wrapper
	 */
	public class ReaderWriterDictionary<TKey, TValue>
	{
		/**
		 * The key to protecting the contained dictionary properly
		 */
		private ReaderWriterLock AccessLock = new ReaderWriterLock();

		/**
		 * The protected dictionary
		 */
		private Dictionary<TKey, TValue> ProtectedDictionary = new Dictionary<TKey,TValue>();

		/**
		 * Wrappers around each method we need to protect in the dictionary
		 */
		public bool Add( TKey K, TValue V )
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock( Timeout.Infinite );
			bool ReturnValue = false;
			try
			{
				// Add a check for whether we're about to add a duplicate key
				if( !ProtectedDictionary.ContainsKey( K ) )
				{
					ProtectedDictionary.Add( K, V );
					ReturnValue = true;
				}
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
			return ReturnValue;
		}

		public bool Remove( TKey K )
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock( Timeout.Infinite );
			bool ReturnValue = false;
			try
			{
				ReturnValue = ProtectedDictionary.Remove( K );
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
			return ReturnValue;
		}

		public void Clear()
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock( Timeout.Infinite );
			try
			{
				ProtectedDictionary.Clear();
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
		}

		public bool ContainsKey( TKey K )
		{
			// Does not modify the collection, use a reader lock
			AccessLock.AcquireReaderLock( Timeout.Infinite );
			bool ReturnValue = false;
			try
			{
				ReturnValue = ProtectedDictionary.ContainsKey( K );
			}
			finally
			{
				AccessLock.ReleaseReaderLock();
			}
			return ReturnValue;
		}

		public bool TryGetValue( TKey K, out TValue V )
		{
			// Does not modify the collection, use a reader lock
			AccessLock.AcquireReaderLock( Timeout.Infinite );
			bool ReturnValue = false;
			try
			{
				ReturnValue = ProtectedDictionary.TryGetValue( K, out V );
			}
			finally
			{
				AccessLock.ReleaseReaderLock();
			}
			return ReturnValue;
		}

		public List<TValue> Values
		{
			get
			{
				// Does not modify the collection, use a reader lock
				AccessLock.AcquireReaderLock( Timeout.Infinite );
				List<TValue> CopyOfValues = null;
				try
				{
					CopyOfValues = new List<TValue>( ProtectedDictionary.Values );
				}
				finally
				{
					AccessLock.ReleaseReaderLock();
				}
				return CopyOfValues;
			}
		}

		public List<TKey> Keys
		{
			get
			{
				// Does not modify the collection, use a reader lock
				AccessLock.AcquireReaderLock( Timeout.Infinite );
				List<TKey> CopyOfKeys = null;
				try
				{
					CopyOfKeys = new List<TKey>( ProtectedDictionary.Keys );
				}
				finally
				{
					AccessLock.ReleaseReaderLock();
				}
				return CopyOfKeys;
			}
		}

		public int Count
		{
			get
			{
				// Does not modify the collection, use a reader lock
				AccessLock.AcquireReaderLock( Timeout.Infinite );
				int ReturnValue = 0;
				try
				{
					ReturnValue = ProtectedDictionary.Count;
				}
				finally
				{
					AccessLock.ReleaseReaderLock();
				}
				return ReturnValue;
			}
		}

		public Dictionary<TKey, TValue> Copy()
		{
			// Does not modify the collection, use a reader lock
			AccessLock.AcquireReaderLock( Timeout.Infinite );
			Dictionary<TKey, TValue> CopyOfDictionary = null;
			try
			{
				CopyOfDictionary = new Dictionary<TKey, TValue>( ProtectedDictionary );
			}
			finally
			{
				AccessLock.ReleaseReaderLock();
			}
			return CopyOfDictionary;
		}
	}

    ///////////////////////////////////////////////////////////////////////////

    /**
     * A thread-safe reader-writer-locked Queue wrapper
     */
    public class ReaderWriterQueue<TValue>
    {
        /**
         * The key to protecting the contained queue properly
         */
        private ReaderWriterLock AccessLock = new ReaderWriterLock();

        /**
         * The protected queue
         */
        private Queue<TValue> ProtectedQueue = new Queue<TValue>();

        /**
         * Wrappers around each method we need to protect in the queue
         */
		public void Clear()
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock( Timeout.Infinite );
			try
			{
				ProtectedQueue.Clear();
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
		}
		
		public void Enqueue( TValue V )
        {
            // Modifies the collection, use a writer lock
            AccessLock.AcquireWriterLock( Timeout.Infinite );
            try
            {
                ProtectedQueue.Enqueue( V );
            }
            finally
            {
                AccessLock.ReleaseWriterLock();
            }
        }

        public TValue Dequeue()
        {
            // Modifies the collection, use a writer lock
            AccessLock.AcquireWriterLock( Timeout.Infinite );
            TValue V = default( TValue );
            try
            {
                V = ProtectedQueue.Dequeue();
            }
            finally
            {
                AccessLock.ReleaseWriterLock();
            }

            return ( V );
        }

		public TValue[] ToArray()
		{
			// Does not modify the collection, use a reader lock
			AccessLock.AcquireReaderLock( Timeout.Infinite );
			TValue[] ReturnValues;
			try
			{
				ReturnValues = ProtectedQueue.ToArray();
			}
			finally
			{
				AccessLock.ReleaseReaderLock();
			}
			return ReturnValues;
		}

        public int Count
        {
            get
            {
                // Does not modify the collection, use a reader lock
                AccessLock.AcquireReaderLock( Timeout.Infinite );
                int ReturnValue = 0;
                try
                {
                    ReturnValue = ProtectedQueue.Count;
                }
                finally
                {
                    AccessLock.ReleaseReaderLock();
                }
                return ReturnValue;
            }
        }
    }

	///////////////////////////////////////////////////////////////////////////

	/**
	 * A thread-safe reader-writer-locked Stack wrapper
	 */
	public class ReaderWriterStack<TValue>
	{
		/**
		 * The key to protecting the contained queue properly
		 */
		private ReaderWriterLock AccessLock = new ReaderWriterLock();

		/**
		 * The protected queue
		 */
		private Stack<TValue> ProtectedStack = new Stack<TValue>();

		/**
		 * Wrappers around each method we need to protect in the queue
		 */
		public void Clear()
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock( Timeout.Infinite );
			try
			{
				ProtectedStack.Clear();
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
		}

		public void Push( TValue V )
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock( Timeout.Infinite );
			try
			{
				ProtectedStack.Push( V );
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
		}

		public TValue Pop()
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock( Timeout.Infinite );
			TValue V = default( TValue );
			try
			{
				V = ProtectedStack.Pop();
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}

			return ( V );
		}

		public TValue[] ToArray()
		{
			// Does not modify the collection, use a reader lock
			AccessLock.AcquireReaderLock( Timeout.Infinite );
			TValue[] ReturnValues;
			try
			{
				ReturnValues = ProtectedStack.ToArray();
			}
			finally
			{
				AccessLock.ReleaseReaderLock();
			}
			return ReturnValues;
		}

		public int Count
		{
			get
			{
				// Does not modify the collection, use a reader lock
				AccessLock.AcquireReaderLock( Timeout.Infinite );
				int ReturnValue = 0;
				try
				{
					ReturnValue = ProtectedStack.Count;
				}
				finally
				{
					AccessLock.ReleaseReaderLock();
				}
				return ReturnValue;
			}
		}
	}
	
	///////////////////////////////////////////////////////////////////////////

    public class PerfTimer
    {
        private class PerfTiming
	    {
		    public string	        Name;
		    public Stopwatch	    StopWatchTimer;
		    public Int32			Count;
		    public bool 			Accumulate;
		    public Int64			Counter;

		    public PerfTiming( string InName, bool InAccumulate )
		    {
			    Name = InName;
			    StopWatchTimer = new Stopwatch();
			    Count = 0;
			    Accumulate = InAccumulate;
			    Counter = 0;
		    }

            public void Start()
		    {
			    StopWatchTimer.Start();
		    }

            public void Stop()					
		    {
			    Count++;
			    StopWatchTimer.Stop();
		    }

            public void IncrementCounter( Int64 Adder )
		    {
			    Counter += Adder;
		    }
	    }

        ReaderWriterDictionary<string, PerfTiming> Timings = new ReaderWriterDictionary<string, PerfTiming>();
        Stack<PerfTiming> LastTimers = new Stack<PerfTiming>();

        public void Start( string Name, bool Accum, Int64 Adder )
        {
            lock( LastTimers )
            {
                PerfTiming Timing = null;
                if( !Timings.TryGetValue( Name, out Timing ) )
                {
                    Timing = new PerfTiming( Name, Accum );
                    Timings.Add( Name, Timing );
                }

                LastTimers.Push( Timing );

                Timing.IncrementCounter( Adder );
                Timing.Start();
            }
        }

        public void Stop()
        {
            lock( LastTimers )
            {
                PerfTiming Timing = LastTimers.Pop();
                Timing.Stop();
            }
        }

        public string DumpTimings()
        {
            string Output = "Machine: " + System.Net.Dns.GetHostName() + Environment.NewLine + Environment.NewLine;

    		double TotalTime = 0.0;
	    	foreach( PerfTiming Timing in Timings.Values )
		    {
			    if( Timing.Count > 0 )
			    {
				    double Elapsed = Timing.StopWatchTimer.Elapsed.TotalSeconds;
				    double Average = ( Elapsed * 1000000.0 ) / Timing.Count;
				    Output += Timing.Name.PadLeft( 30 ) + " : " + Elapsed.ToString( "F3" ) + "s in " + Timing.Count + " calls (" + Average.ToString( "F0" ) + "us per call)";

				    if( Timing.Counter > 0 )
				    {
					    Output += " (" + ( Timing.Counter / 1024 ) + "k)";
				    }

				    Output += Environment.NewLine;
				    if( Timing.Accumulate )
				    {
					    TotalTime += Elapsed;
				    }
			    }
		    }

	    	Output += "\nTotal time inside Swarm: " + TotalTime.ToString( "F3" ) + "s" + Environment.NewLine;

	    	return( Output );
        }
    }

	///////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of Agent-specific utilities
	 */
	public partial class Agent : MarshalByRefObject, IAgentInternalInterface, IAgentInterface
	{
		///////////////////////////////////////////////////////////////////////////

        /*
		 * Logs a line of text to the agent log window
		 */
		public Int32 Log( EVerbosityLevel Verbosity, ELogColour TextColour, string Line )
        {
            AgentApplication.Log( Verbosity, TextColour, Line );
            return ( Constants.SUCCESS );
        }

		///////////////////////////////////////////////////////////////////////////

		private Int32 DirectoryRecreate( string DirectoryName )
		{
			Int32 ErrorCode = Constants.INVALID;

			// Delete the directory, if it exists
			int DeleteTryCount = 0;
			do
			{
				try
				{
					// If this isn't the first time through, sleep a little before trying again
					if( DeleteTryCount > 0 )
					{
						Thread.Sleep( 500 );
					}
					DeleteTryCount++;
					if( Directory.Exists( DirectoryName ) )
					{
						Directory.Delete( DirectoryName, true );
					}
				}
				catch( Exception Ex )
				{
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[DirectoryRecreate] Failed to delete the directory " + DirectoryName + ", there may be files open... retrying" );
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[DirectoryRecreate] Exception was: " + Ex.Message.TrimEnd( '\n' ) );

					ErrorCode = Constants.ERROR_EXCEPTION;
				}
			}
			while( Directory.Exists( DirectoryName ) && ( DeleteTryCount < 10 ) );

			// Create the directory, if it doesn't exist
			int CreateTryCount = 0;
			do
			{
				try
				{
					// If this isn't the first time through, sleep a little before trying again
					if( CreateTryCount > 0 )
					{
						Thread.Sleep( 500 );
					}
					CreateTryCount++;
					if( !Directory.Exists( DirectoryName ) )
					{
						Directory.CreateDirectory( DirectoryName );
					}
				}
				catch( Exception Ex )
				{
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[DirectoryRecreate] Failed to create the directory " + DirectoryName + ", retrying" );
					Log( EVerbosityLevel.Verbose, ELogColour.Red, "[DirectoryRecreate] Exception was: " + Ex.Message.TrimEnd( '\n' ) );

					ErrorCode = Constants.ERROR_EXCEPTION;
				}
			}
			while( !Directory.Exists( DirectoryName ) && ( CreateTryCount < 10 ) );

			// Determine whether we recreated the directory
			if( Directory.Exists( DirectoryName ) )
			{
				ErrorCode = Constants.SUCCESS;
			}

			return ( ErrorCode );
		}

		/**
		 * Helper classes to sort newest timestamp first.
		 */
		public class DirectoryWriteTimestampComparer : IComparer<DirectoryInfo>
		{
			public int Compare( DirectoryInfo A, DirectoryInfo B )
			{
				return -A.LastWriteTimeUtc.CompareTo( B.LastWriteTimeUtc );
			}
		}
		public class FileAccessTimestampComparer : IComparer<FileInfo>
		{
			public int Compare( FileInfo A, FileInfo B )
			{
				return -A.LastAccessTimeUtc.CompareTo( B.LastAccessTimeUtc );
			}
		}

		private Int64 SizeOfDirectory( string DirectoryPath )
		{
			DirectoryInfo DirInfo = new DirectoryInfo( DirectoryPath );
			if( DirInfo.Exists )
			{
				Int64 SummedSize = 0;

				DirectoryInfo[] SubDirectories = DirInfo.GetDirectories();
				foreach( DirectoryInfo NextSubDirectory in SubDirectories )
				{
					// Recur for each subdirectory
					SummedSize += SizeOfDirectory( NextSubDirectory.FullName );
				}

				// Now, sum the file sizes
				FileInfo[] SubDirectoryFiles = DirInfo.GetFiles();
				foreach( FileInfo NextSubDirectoryFile in SubDirectoryFiles )
				{
					SummedSize += NextSubDirectoryFile.Length;
				}

				return SummedSize;
			}
			return 0;
		}

		/**
		 * Deletes all sub-directories except the newest ones, based on 'Date modified'.
		 * @param DirectoryPath				Path to the directory to clean up
		 * @param NumSubDirectoriesToKeep	Number of sub-directories to keep
		 * @param OldestJobKeptUtc			Output param with the creation DateTime of the oldest job directory that was kept
		 * @return The size, in bytes of the resulting directory
		 */
		private void CleanupJobDirectory( string DirectoryPath, Int32 NumSubDirectoriesToKeep, out DateTime OldestJobKeptUtc )
		{
			Log( EVerbosityLevel.Verbose, ELogColour.Green, "[CleanupJobDirectory] Cleaning up jobs directory" );

			OldestJobKeptUtc = new DateTime();
			DirectoryInfo DirInfo = new DirectoryInfo( DirectoryPath );
			if( DirInfo.Exists )
			{
				DirectoryInfo[] SubDirectories = DirInfo.GetDirectories();
				if( SubDirectories.Length > NumSubDirectoriesToKeep )
				{
					// Sort with newest directories first.
					Array.Sort( SubDirectories, new DirectoryWriteTimestampComparer() );

					// Delete all directories except the first NumSubDirectoriesToKeep directories
					// and the special debugging directory used by Lightmass
					string SpecialDebuggingJobDirectory = Path.Combine( DirectoryPath, "Job-00000123-00004567-000089AB-0000CDEF" );
					for( int Index = NumSubDirectoriesToKeep; Index < SubDirectories.Length; ++Index )
					{
						if( SubDirectories[Index].FullName != SpecialDebuggingJobDirectory )
						{
							try
							{
								Directory.Delete( SubDirectories[Index].FullName, true );
							}
							catch( Exception Ex )
							{
								Log( EVerbosityLevel.Verbose, ELogColour.Red, "[CleanupJobDirectory] Error cleaning up " + SubDirectories[Index].FullName );
								Log( EVerbosityLevel.Verbose, ELogColour.Red, "[CleanupJobDirectory] Exception: " + Ex.Message );
							}
						}
					}
				}

				// Refresh the list and get the creation time of the oldest job directory
				// Also, add up sizes of the directories
				SubDirectories = DirInfo.GetDirectories();
				if( SubDirectories.Length > 0 )
				{
					Array.Sort( SubDirectories, new DirectoryWriteTimestampComparer() );
					OldestJobKeptUtc = SubDirectories[SubDirectories.Length - 1].CreationTimeUtc;
				}
			}

			Log( EVerbosityLevel.Verbose, ELogColour.Green, "[CleanupJobDirectory] Done cleaning up jobs directory" );
		}

		/**
		 * Deletes all sub-directories except the newest ones, based on 'Date modified'.
		 * @param DirectoryPath Path to the directory to clean up
		 * @param OldestToKeepUtc The DateTime (UTC) of the oldest log to keep
		 * @return The size, in bytes of the resulting directory
		 */
		private void CleanupLogDirectory( string DirectoryPath, DateTime OldestToKeepUtc )
		{
			Log( EVerbosityLevel.Verbose, ELogColour.Green, "[CleanupJobDirectory] Cleaning up logs directory" );

			DirectoryInfo DirInfo = new DirectoryInfo( DirectoryPath );
			if( DirInfo.Exists )
			{
				FileInfo[] LogFiles = DirInfo.GetFiles();
				if( LogFiles.Length > 0 )
				{
					// For any file older than the specified time, delete it
					foreach( FileInfo NextLogFile in LogFiles )
					{
						if( NextLogFile.LastWriteTimeUtc < OldestToKeepUtc )
						{
							try
							{
								File.Delete( NextLogFile.FullName );
							}
							catch( Exception Ex )
							{
								Log( EVerbosityLevel.Verbose, ELogColour.Red, "[CleanupLogDirectory] Error cleaning up " + NextLogFile.FullName );
								Log( EVerbosityLevel.Verbose, ELogColour.Red, "[CleanupLogDirectory] Exception: " + Ex.Message );
							}
						}
					}
				}
			}

			Log( EVerbosityLevel.Verbose, ELogColour.Green, "[CleanupJobDirectory] Done cleaning up logs directory" );
		}

        private Int32 EnsureFolderExists( string Folder )
        {
            Int32 ErrorCode = Constants.INVALID;
            try
            {
                if( !Directory.Exists( Folder ) )
                {
                    // Create the directory
                    Directory.CreateDirectory( Folder );
                }
            }
            catch( Exception Ex )
            {
                Log( EVerbosityLevel.Verbose, ELogColour.Red, "[EnsureFolderExists] Error: " + Ex.ToString() );
            }

            if( Directory.Exists( Folder )  )
            {
                ErrorCode = Constants.SUCCESS;
            }

            return ( ErrorCode );
        }

        /**
         * An instance of the timing object if timings are being collected
         */
        private ELogFlags LoggingFlags = ELogFlags.LOG_NONE;
		private PerfTimer PerfTimerInstance = new PerfTimer();
		private bool bPerfTimerInstanceActive = false;

        private void CreateTimings( ELogFlags InLoggingFlags )
        {
            LoggingFlags = InLoggingFlags;
            if( ( LoggingFlags & ELogFlags.LOG_TIMINGS ) != 0 )
            {
                bPerfTimerInstanceActive = true;
            }
        }

        /*
         * Start a timer going
         */
        private void StartTiming( string Name, bool Accum, Int64 Adder )
        {
			if( bPerfTimerInstanceActive )
	        {
				lock( PerfTimerInstance )
				{
					PerfTimerInstance.Start( Name, Accum, Adder );
				}
	        }
        }

        private void StartTiming( string Name, bool Accum )
        {
	        StartTiming( Name, Accum, 0 );
        }

        /*
         * Stop a timer
         */
        private void StopTiming()
        {
			if( bPerfTimerInstanceActive )
	        {
				lock( PerfTimerInstance )
				{
					PerfTimerInstance.Stop();
				}
	        }
        }

        /*
         * Print out all the timing info
         */
        private void DumpTimings( Connection Parent )
        {
			lock( PerfTimerInstance )
			{
				if( bPerfTimerInstanceActive )
				{
					string PerfMessage = PerfTimerInstance.DumpTimings();
					Log( EVerbosityLevel.Simple, ELogColour.Blue, PerfMessage );

					AgentInfoMessage Info = new AgentInfoMessage();
					Info.TextMessage = PerfMessage;
					SendMessageInternal( Parent, Info );
				}
			}
        }
	}
}

