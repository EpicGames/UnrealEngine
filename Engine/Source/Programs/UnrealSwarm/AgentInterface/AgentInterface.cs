// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Threading;

namespace AgentInterface
{
	/**
	 * Any globally special values used in Swarm
	 */
	[Serializable]
	public class Constants
	{
		public const Int32 SUCCESS							= 0;
		public const Int32 INVALID							= -1;
		public const Int32 ERROR_FILE_FOUND_NOT				= -2;
		public const Int32 ERROR_NULL_POINTER				= -3;
		public const Int32 ERROR_EXCEPTION					= -4;
		public const Int32 ERROR_INVALID_ARG				= -5;
		public const Int32 ERROR_INVALID_ARG1				= -6;
		public const Int32 ERROR_INVALID_ARG2				= -7;
		public const Int32 ERROR_INVALID_ARG3				= -8;
		public const Int32 ERROR_INVALID_ARG4				= -9;
		public const Int32 ERROR_CHANNEL_NOT_FOUND			= -10;
		public const Int32 ERROR_CHANNEL_NOT_READY			= -11;
		public const Int32 ERROR_CHANNEL_IO_FAILED			= -12;
		public const Int32 ERROR_CONNECTION_NOT_FOUND		= -13;
		public const Int32 ERROR_JOB_NOT_FOUND				= -14;
		public const Int32 ERROR_JOB						= -15;
		public const Int32 ERROR_CONNECTION_DISCONNECTED	= -16;
		public const Int32 ERROR_AGENT_NOT_FOUND			= -17;
	}

	/**
	 * Consistent version enum used by messages, Jobs, Tasks, etc.
	 */
	[Serializable]
	public enum ESwarmVersionValue
	{
		INVALID						= 0x00000000,
		VER_1_0						= 0x00000010,
	}

	/**
	 * Flags to determine the level of logging
	 */
	[Serializable]
	public enum ELogFlags
	{
		LOG_NONE						= 0,
		LOG_TIMINGS						= ( 1 << 0 ),
		LOG_CONNECTIONS					= ( 1 << 1 ),
		LOG_CHANNELS					= ( 1 << 2 ),
		LOG_MESSAGES					= ( 1 << 3 ),
		LOG_JOBS						= ( 1 << 4 ),
		LOG_TASKS						= ( 1 << 5 ),
		LOG_ALL							= LOG_TIMINGS | LOG_CONNECTIONS | LOG_CHANNELS | LOG_MESSAGES | LOG_JOBS | LOG_TASKS
	}

	/**
	 * The level of debug info spewed to the log files
	 */
	[Serializable]
	public enum EVerbosityLevel
	{
		Silent							= 0,
		Critical,
		Simple,
		Informative,
		Complex,
		Verbose,
		ExtraVerbose,
		SuperVerbose
	}

	/// <summary>
	/// Defines the colour of any log text written into the Agent text log.
	/// </summary>
	[Serializable]
	public enum ELogColour
	{
		Green = 0,
		Red,
		Blue,
		Orange
	}


	/**
	 * The current state of the lighting build process
	 */
	[Serializable]
	public enum EProgressionState
	{
		TaskTotal						= 0,
		TasksInProgress,
		TasksCompleted,
		Idle,
		InstigatorConnected,
		RemoteConnected,
		Exporting,
		BeginJob,
		Blocked,
		Preparing0,
		Preparing1,
		Preparing2,
		Preparing3,
		Processing0,
		FinishedProcessing0,
		Processing1,
		FinishedProcessing1,
		Processing2,
		FinishedProcessing2,
		Processing3,
		FinishedProcessing3,
		ExportingResults,
		ImportingResults,
		Finished,
		RemoteDisconnected,
		InstigatorDisconnected,
        Preparing4,
        Num
	}

	/**
	 * Flags that define the intended behavior of the channel. The most
	 * important of which are whether the channel is read or write, and
	 * whether it's a general, persistent cache channel, or whether it's
	 * a job-specific channel. Additional misc flags are available as
	 * well.
	 */
	[Serializable]
	public enum EChannelFlags
	{
		TYPE_PERSISTENT				= 0x00000001,
		TYPE_JOB_ONLY				= 0x00000002,
		TYPE_MASK					= 0x0000000F,

		ACCESS_READ					= 0x00000010,
		ACCESS_WRITE				= 0x00000020,
		ACCESS_MASK					= 0x000000F0,

		// The combinations we care about are easiest to just special case here
		CHANNEL_READ				= TYPE_PERSISTENT | ACCESS_READ,
		CHANNEL_WRITE				= TYPE_PERSISTENT | ACCESS_WRITE,
		JOB_CHANNEL_READ			= TYPE_JOB_ONLY | ACCESS_READ,
		JOB_CHANNEL_WRITE			= TYPE_JOB_ONLY | ACCESS_WRITE,

		// Any additional flags for debugging or extended features
		MISC_ENABLE_PAPER_TRAIL		= 0x00010000,
		MISC_ENABLE_COMPRESSION		= 0x00020000,
		MISC_MASK					= 0x000F0000
	}

	/**
	 * All the different types of messages the Swarm messaging interface supports
	 */
	[Serializable]
	public enum EMessageType
	{
		NONE						= 0x00000000,
		INFO						= 0x00000001,
		ALERT						= 0x00000002,
		TIMING						= 0x00000003,
		PING						= 0x00000004,
		SIGNAL						= 0x00000005,

		/** Job messages */
		JOB_SPECIFICATION			= 0x00000010,
		JOB_STATE					= 0x00000020,

		/** Task messages */
		TASK_REQUEST				= 0x00000100,
		TASK_REQUEST_RESPONSE		= 0x00000200,
		TASK_STATE					= 0x00000300,

		QUIT						= 0x00008000
	}

	[Serializable]
	public enum ETaskRequestResponseType
	{
		RELEASE						= 0x00000001,
		RESERVATION					= 0x00000002,
		SPECIFICATION				= 0x00000003
	}

	/**
	 * Flags used when creating a Job or Task
	 */
	[Serializable]
	public enum EJobTaskFlags
	{
		FLAG_USE_DEFAULTS			= 0x00000000,
		FLAG_ALLOW_REMOTE			= 0x00000001,
		FLAG_MANUAL_START			= 0x00000002,
		FLAG_64BIT					= 0x00000004,
		FLAG_MINIMIZED				= 0x00000008,

		TASK_FLAG_USE_DEFAULTS		= 0x00000000,
		TASK_FLAG_ALLOW_REMOTE		= 0x00000100
	}

	/**
	 * All possible states a Job or Task can be in
	 */
	[Serializable]
	public enum EJobTaskState
	{
		STATE_INVALID				= 0x00000001,
		STATE_IDLE					= 0x00000002,
		STATE_READY					= 0x00000003,
		STATE_RUNNING				= 0x00000004,
		STATE_COMPLETE_SUCCESS		= 0x00000005,
		STATE_COMPLETE_FAILURE		= 0x00000006,
		STATE_KILLED				= 0x00000007,

		TASK_STATE_INVALID			= 0x00000011,
		TASK_STATE_IDLE				= 0x00000012,
		TASK_STATE_ACCEPTED			= 0x00000013,
		TASK_STATE_REJECTED			= 0x00000014,
		TASK_STATE_RUNNING			= 0x00000015,
		TASK_STATE_COMPLETE_SUCCESS	= 0x00000016,
		TASK_STATE_COMPLETE_FAILURE	= 0x00000017,
		TASK_STATE_KILLED			= 0x00000018
	}

	/**
	 * The Alert levels
	 */
	[Serializable]
	public enum EAlertLevel
	{
		ALERT_INFO					= 0x00000001,
		ALERT_WARNING				= 0x00000002,
		ALERT_ERROR					= 0x00000003,
		ALERT_CRITICAL_ERROR		= 0x00000004
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 * A class encapsulating the GUID of an Agent
	 */
	[Serializable]
	public class AgentGuid : IEquatable<AgentGuid>
	{
		public AgentGuid()
		{
			A = 0;
			B = 0;
			C = 0;
			D = 0;
		}
		
		public AgentGuid(UInt32 InA, UInt32 InB, UInt32 InC, UInt32 InD)
		{
			A = InA;
			B = InB;
			C = InC;
			D = InD;
		}
		
		public virtual bool Equals(AgentGuid Other)
		{
			// Needed for use in Dictionary collections
			return (A == Other.A) &&
			       (B == Other.B) &&
			       (C == Other.C) &&
			       (D == Other.D);
		}
		
		public override int GetHashCode()
		{
			return (int)(A ^ B ^ C ^ D );
		}
		
		public override String ToString()
		{
			return String.Format("{0:X8}-{1:X8}-{2:X8}-{3:X8}", A, B, C, D);
		}
		
		public UInt32 A;
		public UInt32 B;
		public UInt32 C;
		public UInt32 D;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentMessage
	{
		public AgentMessage()
		{
			To = Constants.INVALID;
			From = Constants.INVALID;
			Version = ESwarmVersionValue.VER_1_0;
			Type = EMessageType.NONE;
		}

		public AgentMessage(EMessageType NewType)
		{
			To = Constants.INVALID;
			From = Constants.INVALID;
			Version = ESwarmVersionValue.VER_1_0;
			Type = NewType;
		}

		public AgentMessage(ESwarmVersionValue NewVersion, EMessageType NewType)
		{
			To = Constants.INVALID;
			From = Constants.INVALID;
			Version = NewVersion;
			Type = NewType;
		}
		
		// Handles for the sender and the recipient of this message
		public Int32					To;
		public Int32					From;
		public ESwarmVersionValue		Version;
		public EMessageType				Type;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentInfoMessage : AgentMessage
	{
		public AgentInfoMessage()
			: base(EMessageType.INFO)
		{
		}

		public AgentInfoMessage(String NewTextMessage)
			: base(EMessageType.INFO)
		{
			TextMessage = NewTextMessage;
		}

		// Generic text message for informational purposes
		public String					TextMessage;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentTimingMessage : AgentMessage
	{
		public AgentTimingMessage(EProgressionState NewState, int InThreadNum)
			: base(EMessageType.TIMING)
		{
			State = NewState;
			ThreadNum = InThreadNum;
		}

		// The state the process is transitioning to
		public EProgressionState			State;
		// The thread this state is referring to
		public int							ThreadNum;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentJobMessageBase : AgentMessage
	{
		public AgentJobMessageBase(AgentGuid NewJobGuid, EMessageType NewType)
			: base(NewType)
		{
			JobGuid = NewJobGuid;
		}

		// Basic identifier for the Job
		public AgentGuid JobGuid;
	}

	///////////////////////////////////////////////////////////////////////////

	/**
	 *	A message indicating a) a warning, b) an error, or c) a critical error.
	 *	It includes the Job GUID, the GUID of the item causing the issue, a 
	 *	32-bit field intended to identify the type of the item, and a string
	 *	giving the issue message (which will be localized on the UE3-side of
	 *	things.
	 */
	[Serializable]
	public class AgentAlertMessage : AgentJobMessageBase
	{
		public AgentAlertMessage(AgentGuid NewJobGuid)
			: base(NewJobGuid, EMessageType.ALERT)
		{
		}

		public AgentAlertMessage(AgentGuid NewJobGuid, EAlertLevel NewAlertLevel, AgentGuid NewObjectGuid, Int32 NewTypeId, String NewTextMessage)
			: base(NewJobGuid, EMessageType.ALERT)
		{
			AlertLevel = NewAlertLevel;
			ObjectGuid = NewObjectGuid;
			TypeId = NewTypeId;
			TextMessage = NewTextMessage;
		}

		// The type of alert
		public EAlertLevel		AlertLevel;
		// The identifier for the object that is associated with the issue
		public AgentGuid		ObjectGuid;
		// App-specific identifier for the type of the object
		public Int32			TypeId;
		// Generic text message for informational purposes
		public String			TextMessage;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentJobSpecification
	{
		public AgentJobSpecification(AgentGuid NewJobGuid, EJobTaskFlags NewJobFlags, String JobExecutableName, String JobParameters, List<String> JobRequiredDependencies, List<String> JobOptionalDependencies)
		{
			JobGuid = NewJobGuid;
			JobFlags = NewJobFlags;
			ExecutableName = JobExecutableName;
			Parameters = JobParameters;
			RequiredDependencies = JobRequiredDependencies;
			OptionalDependencies = JobOptionalDependencies;
			DependenciesOriginalNames = null;
		}

		// Basic identifiers for the Job
		public AgentGuid						JobGuid;
		public EJobTaskFlags					JobFlags;

		// Variables to set up and define a Job process
		public String							ExecutableName;
		public String							Parameters;

		// List of required channels for this Job and a mapping from the
		// cached name to the original name for each
		public List<String>						RequiredDependencies;
		public List<String>						OptionalDependencies;
		public Dictionary<String, String>		DependenciesOriginalNames;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentSignalMessage : AgentMessage
	{
		public AgentSignalMessage()
			: base(EMessageType.SIGNAL)
		{
			ResetEvent = new ManualResetEvent(false);
		}

		public ManualResetEvent			ResetEvent;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentTaskRequestResponse : AgentJobMessageBase
	{
		public AgentTaskRequestResponse(AgentGuid TaskJobGuid, ETaskRequestResponseType TaskResponseType)
			: base(TaskJobGuid, EMessageType.TASK_REQUEST_RESPONSE)
		{
			ResponseType = TaskResponseType;
		}

		// The type of response this message is. Subclasses add any additional data.
		public ETaskRequestResponseType	ResponseType;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentTaskSpecification : AgentTaskRequestResponse
	{
		public AgentTaskSpecification(AgentGuid TaskJobGuid, AgentGuid TaskTaskGuid, Int32 TaskTaskFlags, String TaskParameters, Int32 TaskCost, List<String> TaskDependencies)
			: base(TaskJobGuid, ETaskRequestResponseType.SPECIFICATION)
		{
			TaskGuid = TaskTaskGuid;
			TaskFlags = TaskTaskFlags;
			Parameters = TaskParameters;
			Cost = TaskCost;
			Dependencies = TaskDependencies;
			DependenciesOriginalNames = null;
		}

		// The GUID used for identifying the Task being referred to
		public AgentGuid					TaskGuid;
		public Int32						TaskFlags;

		// The Task's parameter string specified with AddTask
		public String						Parameters;

		// The Task's cost, relative to all other Tasks in the same Job, used
		// for even distribution and scheduling
		public Int32						Cost;

		// List of required channels for this Job and a mapping from the
		// cached name to the original name for each
		public List<String>					Dependencies;
		public Dictionary<String, String>	DependenciesOriginalNames;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentJobState : AgentJobMessageBase
	{
		public AgentJobState(AgentGuid NewJobGuid, EJobTaskState NewJobState)
			: base(NewJobGuid, EMessageType.JOB_STATE)
		{
			JobState = NewJobState;
		}

		// Basic properties for the Job
		public EJobTaskState				JobState;
		public String				JobMessage;

		// Additional stats for the Job
		public double				JobRunningTime;
		public Int32				JobExitCode;
	}

	///////////////////////////////////////////////////////////////////////////

	[Serializable]
	public class AgentTaskState : AgentJobMessageBase
	{
		public AgentTaskState(AgentGuid NewJobGuid, AgentGuid NewTaskGuid, EJobTaskState NewTaskState)
			: base(NewJobGuid, EMessageType.TASK_STATE)
		{
			TaskGuid = NewTaskGuid;
			TaskState = NewTaskState;
		}

		// Basic properties for the Task
		public AgentGuid			TaskGuid;
		public EJobTaskState		TaskState;
		public String				TaskMessage;

		// Additional stats for the Task
		public double				TaskRunningTime;
		public Int32				TaskExitCode;
	}

	///////////////////////////////////////////////////////////////////////////
	
	/**
	 * The primary interface to the Swarm system
	 */
	public interface IAgentInterface
	{
		/**
		 * Opens a new local connection to the Swarm
		 */
		Int32 OpenConnection(Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Closes an existing connection to the Swarm, if one is open
		 */
		Int32 CloseConnection(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Sends a message to an Agent (return messages are sent via the FConnectionCallback)
		 */
		Int32 SendMessage(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/*
		 * Gets the next message in the queue, optionally blocking until a message is available
		 * 
		 * TimeOut - Milliseconds to wait (-1 is infinite)
		 */
		Int32 GetMessage(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
	 	 * Adds an existing file to the cache. Note, any existing channel with the same
		 * name will be overwritten.
		 */
		Int32 AddChannel(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Determines if the named channel is in the cache
		 */
		Int32 TestChannel(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Determines if a requested channel is safe to open (returns >= 0 if so)
		 */
		Int32 OpenChannel(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Closes an open channel
		 */
		Int32 CloseChannel(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Opens a Job session, which allows a Job to be specified, Tasks added, Job
		 * channels opened and used, etc. When the Job is complete and no more Job
		 * related data is needed from the Swarm, call CloseJob.
		 */
		Int32 OpenJob(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Begins a Job specification, which allows a series of Tasks to be specified
		 * via AddTask. When Tasks are done being specified, call EndJobSpecification.
		 *
		 * The default behavior will be to execute the Job executable with the
		 * specified parameters. If Tasks are added for the Job, they are expected
		 * to be requested by the executable run for the Job. If no Tasks are added
		 * for the Job, it is expected that the Job executable will perform its
		 * operations without additional Task input from Swarm.
		 */
		Int32 BeginJobSpecification(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Adds a Task to the current Job
		 */
		Int32 AddTask(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Ends the Job specification, after which no additional Tasks may be defined. Also,
		 * this is generally the point when the Agent will validate and launch the Job executable,
		 * potentially distributing the Job to other Agents.
		 */
		Int32 EndJobSpecification(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * Ends the Job, after which all Job-related API usage (except OpenJob) will be rejected
		 */
		Int32 CloseJob(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/**
		 * A fully general interface to the Agent which is used for extending the API in novel
		 * ways, debugging interfaces, extended statistics gathering, etc.
		 */
		Int32 Method(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		
		/*
		 * Logs a line of text to the agent log window
		 */
		Int32 Log(EVerbosityLevel Verbosity, ELogColour TextColour, String Line);
	}
}
