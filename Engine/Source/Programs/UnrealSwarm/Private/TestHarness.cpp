// Copyright Epic Games, Inc. All Rights Reserved.

//
// TestHarness.cpp : Defines the entry point for the console application.
//                   Currently tests the basic API of the SwarmInterface library
//

#include <conio.h>
#include <stdio.h>
#include <tchar.h>
#include <time.h>

#include <Windows.h>
#include "../../../Editor/Windows/UnrealSwarm/Public/SwarmDefines.h"
#include "../../../Editor/Windows/UnrealSwarm/Public/SwarmInterface.h"
using namespace NSwarm;


// The primary system callback
int OutstandingPings = 0;
int OutstandingTasks = 0;

bool JobIsRunning = false;
bool TaskIsRunning = false;
bool ContinueRunning = true;

void MyConnectionCallback( FMessage* CallbackMessage, void* CallbackData )
{
	// Get the global handle to the system
	FSwarmInterface& GSwarmInterface = FSwarmInterface::Get();

	if (CallbackData == NULL)
	{
		printf( "NOTE: Callback data is NULL\n" );
	}
	// TODO: As we add additional versions, convert to a switch rather than if-else.
	// For now, just use a simple if since we only have one version and a switch is
	// overkill.
	if (CallbackMessage->Version == VERSION_1_0)
	{
		switch (CallbackMessage->Type)
		{
			case MESSAGE_PING:
				OutstandingPings--;
				printf( "Ping returned!\n" );
			break;

			case MESSAGE_INFO:
			{
				printf( "*** MessageInfo:\n" );
				FInfoMessage* InfoMessage = (FInfoMessage*)CallbackMessage;
				_tprintf( InfoMessage->TextMessage );
				printf( "\n" );
			}
			break;

			case MESSAGE_ALERT:
			{
				FAlertMessage* AlertMessage = (FAlertMessage*)CallbackMessage;
				printf("Alert Message received:\n");
				printf("\tThreat level (%d)....", (int)(AlertMessage->AlertLevel));
				switch (AlertMessage->AlertLevel)
				{
				case ALERT_LEVEL_INFO:				printf("INFO\n");				break;
				case ALERT_LEVEL_WARNING:			printf("WARNING\n");			break;
				case ALERT_LEVEL_ERROR:				printf("ERROR\n");				break;
				case ALERT_LEVEL_CRITICAL_ERROR:	printf("CRITICAL ERROR\n");		break;
				default:							printf("*** UNKNOWN ***\n");	break;
				}
				printf("\tJob.............{%08x,%08x,%08x,%08x}\n", AlertMessage->JobGuid.A,
																	AlertMessage->JobGuid.B,
																	AlertMessage->JobGuid.C,
																	AlertMessage->JobGuid.D);
				printf("\tObject..........{%08x,%08x,%08x,%08x}\n", AlertMessage->ObjectGuid.A,
																	AlertMessage->ObjectGuid.B,
																	AlertMessage->ObjectGuid.C,
																	AlertMessage->ObjectGuid.D);
				printf("\tTypeId..........%08x\n", AlertMessage->TypeId);
				_tprintf(TEXT("\t%s\n"), AlertMessage->TextMessage);
			}
			break;

			case MESSAGE_QUIT:
				ContinueRunning = false;
				printf( "Quit message received!\n" );
			break;

			case MESSAGE_JOB_STATE:
			{
				FJobState* JobStateMessage = (FJobState*)CallbackMessage;
				printf( "Job  {%08x,%08x,%08x,%08x}\n", JobStateMessage->JobGuid.A,
														JobStateMessage->JobGuid.B,
														JobStateMessage->JobGuid.C,
														JobStateMessage->JobGuid.D );
				switch (JobStateMessage->JobState)
				{
					case JOB_STATE_INVALID:
						printf( "is invalid" );
						ContinueRunning = false;
						break;
					case JOB_STATE_READY:
						printf( "is ready" );
						break;
					case JOB_STATE_RUNNING:
						printf( "is now running" );
						JobIsRunning = true;
						break;
					case JOB_STATE_COMPLETE_SUCCESS:
						printf( "is complete and is successful" );
						JobIsRunning = false;
						OutstandingTasks = 0;
						break;
					case JOB_STATE_COMPLETE_FAILURE:
						printf( "is complete and is a failure with an exit code of %d", JobStateMessage->JobExitCode );
						JobIsRunning = false;
						OutstandingTasks = 0;
						break;
					case JOB_STATE_KILLED:
						printf( "was killed" );
						JobIsRunning = false;
						OutstandingTasks = 0;
						break;
					default:
						printf( "sent an unhandled message" );
						ContinueRunning = false;
						break;
				}
				printf( "\n" );
			}
			break;

			case MESSAGE_TASK_STATE:
			{
				FTaskState* TaskStateMessage = (FTaskState*)CallbackMessage;
				printf( "Task {%08x,%08x,%08x,%08x}\n", TaskStateMessage->TaskGuid.A,
					TaskStateMessage->TaskGuid.B,
					TaskStateMessage->TaskGuid.C,
					TaskStateMessage->TaskGuid.D );
				printf( "    " );
				switch (TaskStateMessage->TaskState)
				{
					case JOB_TASK_STATE_INVALID:
						printf( "is invalid" );
						ContinueRunning = false;
						break;
					case JOB_TASK_STATE_ACCEPTED:
						printf( "was accepted" );
						TaskIsRunning = true;
						break;
					case JOB_TASK_STATE_REJECTED:
						printf( "was rejected" );
						TaskIsRunning = false;
						break;
					case JOB_TASK_STATE_RUNNING:
						printf( "is now running" );
						TaskIsRunning = true;
						break;
					case JOB_TASK_STATE_COMPLETE_SUCCESS:
						printf( "is complete and is successful" );
						OutstandingTasks--;
						TaskIsRunning = false;
						break;
					case JOB_TASK_STATE_COMPLETE_FAILURE:
						printf( "is complete and is a failure" );
						OutstandingTasks--;
						TaskIsRunning = false;
						break;
					case JOB_TASK_STATE_KILLED:
						printf( "was killed" );
						TaskIsRunning = false;
						break;
					default:
						printf( "sent an unhandled message" );
						ContinueRunning = false;
						break;
				}
				printf( "\n" );
			}
			break;

			case MESSAGE_TASK_REQUEST_RESPONSE:
			{
				FTaskRequestResponse* TaskRequestResponse = (FTaskRequestResponse*)CallbackMessage;

				switch (TaskRequestResponse->ResponseType)
				{
					case RESPONSE_TYPE_RELEASE:
						printf( "    " );
						printf( "has released us from future Tasks\n" );
						JobIsRunning = false;
						OutstandingTasks = 0;
					break;

					case RESPONSE_TYPE_RESERVATION:
					{
						printf( "    " );
						printf( "has requested a reservation for future Tasks\n" );
						JobIsRunning = true;
					}
					break;

					case RESPONSE_TYPE_SPECIFICATION:
					{
						FTaskSpecification* TaskSpecification = (FTaskSpecification*)TaskRequestResponse;
						FGuid TaskGuid = TaskSpecification->TaskGuid;

						printf( "Task {%08x,%08x,%08x,%08x}\n", TaskGuid.A,
							TaskGuid.B,
							TaskGuid.C,
							TaskGuid.D );
						printf( "    " );
						
						// Accept the Task and say that it's now running
						GSwarmInterface.SendMessage( FTaskState( TaskGuid, JOB_TASK_STATE_ACCEPTED ) );
						GSwarmInterface.SendMessage( FTaskState( TaskGuid, JOB_TASK_STATE_RUNNING ) );

						// Do the Task (of printing the parameters out)
						printf( "Parameters: %S\n", TaskSpecification->Parameters );

						// Say the Task is complete
						GSwarmInterface.SendMessage( FTaskState( TaskGuid, JOB_TASK_STATE_COMPLETE_SUCCESS ) );
					}
					break;
				}
			}
		}
	}
}

void SwarmAwareTestPathServer( _TCHAR* ExecutableName )
{
	// Get the global handle to the system
	FSwarmInterface& GSwarmInterface = FSwarmInterface::Get();

	// Open a connection with the system
	INT ErrorCode = GSwarmInterface.OpenConnection( MyConnectionCallback, NULL, SWARM_LOG_NONE );
	if( ErrorCode < 0 )
	{
		// Error
		printf( "******** Error, OpenConnection failed! (%d)\n", ErrorCode );
	}

	// Create the Job Guid for our work
	FGuid JobGuid( 0x2, 0x23, 0x11, 0x17 );

	// Start a Job and add some Tasks
	JobIsRunning = true;

	ErrorCode = GSwarmInterface.OpenJob( JobGuid );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, OpenJob failed! (%d)\n", ErrorCode );
	}

	// Write out some sample data to test bi-direction file communication
	{
		INT FileHandle = GSwarmInterface.OpenChannel( L"ServerRandomNumbers.txt", SWARM_JOB_CHANNEL_WRITE );
		if( FileHandle < 0 )
		{
			// Error.
			printf( "******** Error, OpenChannel failed! (%d)\n", FileHandle );
		}

		// Some random data to a "well known file", to be read in the client
		srand( ( unsigned )time( NULL ) );
		int RandomNumber0 = rand();
		int RandomNumber1 = rand();
		int RandomNumber2 = rand();

		printf( "Server writes data: %d, %d, %d\n", RandomNumber0, RandomNumber1, RandomNumber2 );
		ErrorCode = GSwarmInterface.WriteChannel( FileHandle, &RandomNumber0, sizeof( RandomNumber0 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, WriteChannel error! (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.WriteChannel( FileHandle, &RandomNumber1, sizeof( RandomNumber1 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, WriteChannel error! (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.WriteChannel( FileHandle, &RandomNumber2, sizeof( RandomNumber2 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, WriteChannel error! (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.CloseChannel( FileHandle );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, CloseChannel error! (%d)\n", ErrorCode );
		}
	}

	// Specify the Job
	FJobSpecification NewJobSpecification( ExecutableName,
										   L"-swarmaware-client 2 23 11 17",
										   JOB_FLAG_ALLOW_REMOTE );
	
	const _TCHAR** NewJobDependencies = new const _TCHAR*[1];
	NewJobDependencies[0] = L"AgentInterface.dll";
	NewJobSpecification.AddDependencies( NewJobDependencies, 1, NULL, 0 );

	ErrorCode = GSwarmInterface.BeginJobSpecification( NewJobSpecification, FJobSpecification() );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, BeginJobSpecification failed! (%d)\n", ErrorCode );
	}

	delete[] NewJobDependencies;

	// Add Tasks to the Job
	{
		FGuid TaskGuid0( 0x23, 0x11, 0x17, 0x2 );
		FTaskSpecification NewTask = FTaskSpecification( TaskGuid0, L"This is Task number 1", JOB_TASK_FLAG_USE_DEFAULTS );

		const _TCHAR** NewTaskDependencies = new const _TCHAR*[1];
		NewTaskDependencies[0] = L"AgentInterface.dll";
		NewTask.AddDependencies( NewTaskDependencies, 1 );

		ErrorCode = GSwarmInterface.AddTask( NewTask );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask1 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid1( 0x11, 0x17, 0x2, 0x23 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid1, L"This is Task number 2", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask2 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid2( 0x17, 0x2, 0x23, 0x11 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid2, L"This is Task number 3", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask3 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid3( 0x2, 0x23, 0x11, 0x17 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid3, L"This is Task number 4", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask4 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid4( 0x18, 0x3, 0x24, 0x12 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid4, L"This is Task number 5", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask5 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid5( 0x3, 0x24, 0x12, 0x18 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid5, L"This is Task number 6", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask6 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid6( 0x24, 0x12, 0x18, 0x3 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid6, L"This is Task number 7", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask7 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid7( 0x12, 0x18, 0x3, 0x24 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid7, L"This is Task number 8", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask8 failed! (%d)\n", ErrorCode );
		}

		FGuid TaskGuid8( 0x12, 0x18, 0x3, 0x25 );
		ErrorCode = GSwarmInterface.AddTask( FTaskSpecification( TaskGuid8, L"This is Task number 9", JOB_TASK_FLAG_USE_DEFAULTS ) );
		OutstandingTasks++;
		if( ErrorCode < 0 )
		{
			printf( "******** Error, AddTask9 failed! (%d)\n", ErrorCode );
		}
	}

	ErrorCode = GSwarmInterface.EndJobSpecification();
	if( ErrorCode < 0 )
	{
		printf( "******** Error, EndJobSpecification failed! (%d)\n", ErrorCode );
	}

	// Wait for all pings to return
	while( ContinueRunning && ( OutstandingTasks > 0 ) )
	{
		// Yield
		printf( "Waiting for quit signal... %d\n", OutstandingTasks );
		Sleep( 1000 );
	}

	// Open and read the data written by the client from the "well known file"
	{
		INT FileHandle = GSwarmInterface.OpenChannel( L"ClientRandomNumbers.txt", SWARM_JOB_CHANNEL_READ );
		if( FileHandle < 0 )
		{
			// Error.
			printf( "******** Error, OpenChannel failed! (%d)\n", FileHandle );
		}

		int RandomNumber0 = 0;
		int RandomNumber1 = 0;
		int RandomNumber2 = 0;

		ErrorCode = GSwarmInterface.ReadChannel( FileHandle, &RandomNumber0, sizeof( RandomNumber0 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, ReadChannel error! (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.ReadChannel( FileHandle, &RandomNumber1, sizeof( RandomNumber1 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, ReadChannel error! (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.ReadChannel( FileHandle, &RandomNumber2, sizeof( RandomNumber2 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, ReadChannel error! (%d)\n", ErrorCode );
		}

		printf( "Client reads data: %d, %d, %d\n", RandomNumber0, RandomNumber1, RandomNumber2 );

		ErrorCode = GSwarmInterface.CloseChannel( FileHandle );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, CloseChannel error! (%d)\n", ErrorCode );
		}
	}

	// End the Job
	ErrorCode = GSwarmInterface.CloseJob();
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseJob failed! (%d)\n", ErrorCode );
	}

	// Close the connection
	printf( "Closing the connection and quitting\n" );
	ErrorCode = GSwarmInterface.CloseConnection();
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseConnection failed! (%d)\n", ErrorCode );
	}
}

void SwarmAwareTestPathClient( _TCHAR* Guid0, _TCHAR* Guid1, _TCHAR* Guid2, _TCHAR* Guid3 )
{
	INT ErrorCode = SWARM_INVALID;

	// Get the global handle to the system
	FSwarmInterface& GSwarmInterface = FSwarmInterface::Get();

	// Open a connection with the system
	ErrorCode = GSwarmInterface.OpenConnection( MyConnectionCallback, NULL, SWARM_LOG_NONE );
	if( ErrorCode < 0 )
	{
		// Error
		printf( "******** Error, OpenConnection failed! (%d)\n", ErrorCode );
	}

	// Create the Job Guid for our work
	FGuid JobGuid;
	swscanf_s( Guid0, L"%08x", &JobGuid.A, sizeof( JobGuid.A ) );
	swscanf_s( Guid1, L"%08x", &JobGuid.B, sizeof( JobGuid.B ) );
	swscanf_s( Guid2, L"%08x", &JobGuid.C, sizeof( JobGuid.C ) );
	swscanf_s( Guid3, L"%08x", &JobGuid.D, sizeof( JobGuid.D ) );

	// Open and read the data written by the server from the "well known file"
	{
		INT FileHandle = GSwarmInterface.OpenChannel( L"ServerRandomNumbers.txt", SWARM_JOB_CHANNEL_READ );
		if( FileHandle < 0 )
		{
			// Error.
			printf( "******** Error, OpenChannel failed! (%d)\n", FileHandle );
		}

		int RandomNumber0 = 0;
		int RandomNumber1 = 0;
		int RandomNumber2 = 0;
		ErrorCode = GSwarmInterface.ReadChannel( FileHandle, &RandomNumber0, sizeof( RandomNumber0 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, ReadChannel error (%d)\n", ErrorCode );
		}

		ErrorCode =  GSwarmInterface.ReadChannel( FileHandle, &RandomNumber1, sizeof( RandomNumber1 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, ReadChannel error (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.ReadChannel( FileHandle, &RandomNumber2, sizeof( RandomNumber2 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, ReadChannel error (%d)\n", ErrorCode );
		}

		printf( "Client reads data: %d, %d, %d\n", RandomNumber0, RandomNumber1, RandomNumber2 );

		ErrorCode = GSwarmInterface.CloseChannel( FileHandle );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, CloseChannel error (%d)\n", ErrorCode );
		}
	}

	// Write out some sample data to test bi-direction file communication
	{
		INT FileHandle = GSwarmInterface.OpenChannel( L"ClientRandomNumbers.txt", SWARM_JOB_CHANNEL_WRITE );
		if( FileHandle < 0 )
		{
			// Error.
			printf( "******** Error, OpenChannel failed! (%d)\n", FileHandle );
		}

		// Some random data to a "well known file", to be read in the client
		srand( ( unsigned )time(NULL) );
		int RandomNumber0 = rand();
		int RandomNumber1 = rand();
		int RandomNumber2 = rand();

		printf( "Client writes data: %d, %d, %d\n", RandomNumber0, RandomNumber1, RandomNumber2 );
		ErrorCode = GSwarmInterface.WriteChannel( FileHandle, &RandomNumber0, sizeof( RandomNumber0 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, WriteChannel error (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.WriteChannel( FileHandle, &RandomNumber1, sizeof( RandomNumber1 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, WriteChannel error (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.WriteChannel( FileHandle, &RandomNumber2, sizeof( RandomNumber2 ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, WriteChannel error (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.CloseChannel( FileHandle );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, CloseChannel error (%d)\n", ErrorCode );
		}
	}

	// Test the info message
	{
		GSwarmInterface.SendMessage(FInfoMessage(TEXT("This is an info message test!")));
	}

	// Test the alert message
	{
		FGuid ObjectGuid;
		ObjectGuid = FGuid(0,0,0,1);
		GSwarmInterface.SendMessage(FAlertMessage(JobGuid, NSwarm::ALERT_LEVEL_INFO,			ObjectGuid, 0x00000001, TEXT("This is an alert: INFO")));
		ObjectGuid = FGuid(0,0,1,0);
		GSwarmInterface.SendMessage(FAlertMessage(JobGuid, NSwarm::ALERT_LEVEL_WARNING,			ObjectGuid, 0x00000002, TEXT("This is an alert: WARNING")));
		ObjectGuid = FGuid(0,1,0,0);
		GSwarmInterface.SendMessage(FAlertMessage(JobGuid, NSwarm::ALERT_LEVEL_ERROR,			ObjectGuid, 0x00000003, TEXT("This is an alert: ERROR")));
		ObjectGuid = FGuid(1,0,0,0);
		GSwarmInterface.SendMessage(FAlertMessage(JobGuid, NSwarm::ALERT_LEVEL_CRITICAL_ERROR,	ObjectGuid, 0x00000004, TEXT("This is an alert: CRITICAL ERROR")));
	}

	// Run until there is no more work to do
	printf( "Starting to ask for Tasks and will keep running until we get the quit signal\n" );
	JobIsRunning = true;
	while( ContinueRunning && JobIsRunning )
	{
		// Start asking for Tasks
		printf( "Sending request for task\n" );
		ErrorCode = GSwarmInterface.SendMessage( FMessage( MESSAGE_TASK_REQUEST ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Error, SendMessage error (%d)\n", ErrorCode );
		}
		else
		{
			printf( " ...Sent\n" );
		}

		// Yield
		Sleep( 3000 );
	}

	// Close the connection
	printf( "Closing the connection and quitting\n" );
	ErrorCode = GSwarmInterface.CloseConnection();
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseConnection failed! (%d)\n", ErrorCode );
	}
}

void NonSwarmAwareTestPath( _TCHAR* ExecutableName )
{
	// Get the global handle to the system
	FSwarmInterface& GSwarmInterface = FSwarmInterface::Get();

	// Open a connection with the system
	INT ErrorCode = GSwarmInterface.OpenConnection( MyConnectionCallback, NULL, SWARM_LOG_NONE );
	if( ErrorCode < 0 )
	{
		// Error
		printf( "******** Error, OpenConnection failed! (%d)\n", ErrorCode );
	}

	ErrorCode = GSwarmInterface.SendMessage( FInfoMessage( L"Testing info message" ) );
	if( ErrorCode < 0 )
	{
		// Error
		printf( "******** Error, SendMessage failed! (%d)\n", ErrorCode );
	}

	ErrorCode = GSwarmInterface.CloseConnection();
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseConnection failed! (%d)\n", ErrorCode );
	}

	ContinueRunning = true;

	ErrorCode = GSwarmInterface.OpenConnection( MyConnectionCallback, NULL, SWARM_LOG_NONE );
	if( ErrorCode < 0 )
	{
		// Error
		printf( "******** Error, OpenConnection failed! (%d)\n", ErrorCode );
	}

	// Test to see if the channel is already there
	ErrorCode = GSwarmInterface.TestChannel( L"This Is A Test.txt" );
	if( ErrorCode >= 0 )
	{
		printf( "\"This Is A Test.txt\" is already in the cache! (%d)\n", ErrorCode );
	}
	else
	{
		printf( "\"This Is A Test.txt\" is not in the cache! (%d)\n", ErrorCode );
	}

	INT TestFileHandle1 = GSwarmInterface.OpenChannel( L"This Is A Test.txt", SWARM_CHANNEL_WRITE );
	if( TestFileHandle1 < 0 )
	{
		// Error.
		printf( "******** Error, OpenChannel failed! (%d)\n", ErrorCode );
	}

	INT TestFileHandle2 = GSwarmInterface.OpenChannel( L"This Is Another Test.txt", SWARM_CHANNEL_WRITE );
	if( TestFileHandle2 < 0 )
	{
		// Error.
		printf( "******** Error, OpenChannel failed! (%d)\n", ErrorCode );
	}

	char Data1[128] = "Any sufficiently advanced technology is indistinguishable from magic.";
	ErrorCode = GSwarmInterface.WriteChannel( TestFileHandle1, Data1, sizeof( Data1 ) );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, WriteChannel error (%d)\n", ErrorCode );
	}

	char Data2[128] = "Any sufficiently advanced technology is indistinguishable from magic... or is it?";
	ErrorCode = GSwarmInterface.WriteChannel( TestFileHandle2, Data2, sizeof( Data2 ) );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, WriteChannel error (%d)\n", ErrorCode );
	}

	ErrorCode = GSwarmInterface.CloseChannel( TestFileHandle1 );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseChannel error (%d)\n", ErrorCode );
	}

	ErrorCode = GSwarmInterface.CloseChannel( TestFileHandle2 );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseChannel error (%d)\n", ErrorCode );
	}

	// Read and then write some the written data
	INT TestFileHandle3 = GSwarmInterface.OpenChannel( L"This Is Another Test.txt", SWARM_CHANNEL_READ );
	if( TestFileHandle3 < 0 )
	{
		// Error.
		printf( "******** Error, OpenChannel failed! (%d)\n", TestFileHandle3 );
	}

	char Data3[128];
	memset( Data3, 0, sizeof( Data3 ) );
	int Data3ReadSize = GSwarmInterface.ReadChannel( TestFileHandle3, Data3, sizeof( Data3 ) );
	if( Data3ReadSize < 0 )
	{
		printf( "******** Error, ReadChannel error (%d)\n", Data3ReadSize );
	}

	ErrorCode = GSwarmInterface.CloseChannel( TestFileHandle3 );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseChannel error (%d)\n", ErrorCode );
	}

	TestFileHandle3 = GSwarmInterface.OpenChannel( L"This Is A Third Test.txt", SWARM_CHANNEL_WRITE );
	if( TestFileHandle3 < 0 )
	{
		// Error.
		printf( "******** Error, OpenChannel failed! (%d)\n", ErrorCode );
	}

	ErrorCode = GSwarmInterface.WriteChannel( TestFileHandle3, Data3, Data3ReadSize );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, WriteChannel error (%d)\n", ErrorCode );
	}

	ErrorCode = GSwarmInterface.CloseChannel( TestFileHandle3 );
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseChannel error (%d)\n", ErrorCode );
	}

	// Try sending out a few messages
	for( int p = 0; p < 3; p++ )
	{
		ErrorCode = GSwarmInterface.SendMessage( FMessage( MESSAGE_PING ) );
		if( ErrorCode < 0 )
		{
			printf( "******** Ping failed (%d)\n", ErrorCode );
		}
		else
		{
			printf( "Ping sent...\n" );
			OutstandingPings++;
		}
	}

	// Try creating a simple non-Swarm-aware Job
	FGuid JobGuid0( 1, 2, 3, 4 );
	printf( "Beginning Job specification  (expect reject)...\n" );
	FJobSpecification NewJobSpecificationReject( L"notepad.exe", L"", JOB_FLAG_USE_DEFAULTS );
	ErrorCode = GSwarmInterface.OpenJob( JobGuid0 );
	if( ErrorCode >= 0 )
	{
		ErrorCode = GSwarmInterface.BeginJobSpecification( NewJobSpecificationReject, FJobSpecification() );
		if( ErrorCode >= 0 )
		{
			printf( "******** Error, BeginJobSpecification succeeded (and we expected failure)! (%d)\n", ErrorCode );
		}

		ErrorCode = GSwarmInterface.CloseJob();
		if( ErrorCode < 0 )
		{
			printf( "******** Error, CloseJob failed! (%d)\n", ErrorCode );
		}
	}
	else
	{
		printf( "Error, OpenJob failed! (%d)\n", ErrorCode );
	}

	FGuid JobGuid1( 4, 3, 2, 1 );

	ErrorCode = GSwarmInterface.OpenJob( JobGuid1 );
	if( ErrorCode >= 0 )
	{
		printf( "Beginning Job specification (expect accept)...\n" );
		FJobSpecification NewJobSpecificationAccept( ExecutableName, L"-nonswarmaware", JOB_FLAG_USE_DEFAULTS );

		const _TCHAR** NewJobDependencies = new const _TCHAR*[1];
		NewJobDependencies[0] = L"AgentInterface.dll";
		NewJobSpecificationAccept.AddDependencies( NewJobDependencies, 1, NULL, 0 );

		ErrorCode = GSwarmInterface.BeginJobSpecification( NewJobSpecificationAccept, FJobSpecification() );
		if( ErrorCode >= 0 )
		{
			ErrorCode = GSwarmInterface.EndJobSpecification();
			if( ErrorCode < 0 )
			{
				printf( "******** Error, EndJobSpecification failed! (%d)\n", ErrorCode );
			}

			JobIsRunning = true;
		}
		else
		{
			printf( "******** Error, BeginJobSpecification failed! (%d)\n", ErrorCode );
		}
	}
	else
	{
		printf( "******** Error, OpenJob failed! (%d)\n", ErrorCode );
	}

	// Try opening a file for read that doesn't exist (should error)
	INT TestFileHandle4 = GSwarmInterface.OpenChannel( L"This Is A Fourth Test.txt", SWARM_CHANNEL_READ );
	if( TestFileHandle4 >= 0 )
	{
		// Error.
		printf( "******** Error, intentional OpenChannel failure didn't fail! (%d)\n", TestFileHandle4 );
	}
	else
	{
		printf( "Error, intentional OpenChannel failure failed! (%d)\n", TestFileHandle4 );
	}

	// Wait for all pings to return
	while( ContinueRunning && ( ( OutstandingPings > 0 ) || JobIsRunning ) )
	{
		// Yield
		printf( "Waiting for pings to return and tasks to complete...\n" );
		Sleep( 1000 );
	}

	// End the Job
	ErrorCode = GSwarmInterface.CloseJob();
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseJob failed! (%d)\n", ErrorCode );
	}

	// Close the connection
	printf( "Closing the connection and quitting\n" );
	ErrorCode = GSwarmInterface.CloseConnection();
	if( ErrorCode < 0 )
	{
		printf( "******** Error, CloseConnection failed! (%d)\n", ErrorCode );
	}
}

int _tmain(int argc, _TCHAR* argv[])
{
	// If we're only acting a simple task application, just tick and quit
	if( ( argc > 1 ) && ( wcscmp( argv[1], L"-nonswarmaware" ) == 0 ) )
	{
		printf( "Going to sleep for a little while...\n" );
		Sleep( 10000 );
		printf( "Done sleeping, quitting!\n" );
		exit( 0 );
	}

	// If we're acting like a Swarm-aware application, run that path
	if( ( argc > 1 ) && ( wcscmp( argv[1], L"-swarmaware-server") == 0 ) )
	{
		printf( "Starting up Swarm-aware server path\n" );
		SwarmAwareTestPathServer( argv[0] );
	}
	else if( ( argc > 1 ) && ( wcscmp( argv[1], L"-swarmaware-client" ) == 0 ) )
	{
		if( argc >= 5 )
		{
			printf( "Starting up Swarm-aware client path\n" );
			SwarmAwareTestPathClient( argv[2], argv[3], argv[4], argv[5] );
		}
		else
		{
			printf( "Failed to start Swarm-aware path. Not enough parameters...\n" );
		}
	}
	else
	{
		printf( "Starting up non-Swarm-aware path\n" );
		NonSwarmAwareTestPath( argv[0] );
	}

	printf("\n...TestHarness complete...\n");
	printf("Press any key to exit...\n");
	while (!_kbhit())
	{
	}
	return( 0 );
}
