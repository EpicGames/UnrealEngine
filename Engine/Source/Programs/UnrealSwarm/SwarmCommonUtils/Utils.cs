// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.NetworkInformation;
using System.Runtime.InteropServices;
using System.Runtime.Serialization.Formatters.Binary;
using System.Text;
using System.Threading.Tasks;

namespace SwarmCommonUtils
{
	public static class NetworkUtils
	{
		[DllImport("iphlpapi.dll", SetLastError = true, EntryPoint = "GetBestInterface")]
		static extern int WINAPI_GetBestInterface(UInt32 DestAddr, out UInt32 BestIfIndex);

		public static NetworkInterface GetBestInterface(IPAddress Address)
		{
			UInt32 AddressAsUInt32 = BitConverter.ToUInt32(Address.GetAddressBytes(), 0);
			UInt32 InterfaceIndex;

			if (WINAPI_GetBestInterface(AddressAsUInt32, out InterfaceIndex) != 0)
			{
				throw new InvalidOperationException("Couldn't detect best network interface for that address.");
			}

			var Interfaces = NetworkInterface.GetAllNetworkInterfaces().Where(Interface =>
			{
                return Interface.Supports(NetworkInterfaceComponent.IPv4) && Interface.GetIPProperties() != null && Interface.GetIPProperties().GetIPv4Properties() != null && Interface.GetIPProperties().GetIPv4Properties().Index == InterfaceIndex;
			});

			if (Interfaces.Count() == 0)
			{
				throw new InvalidOperationException("Couldn't detect best network interface for that address.");
			}

			return Interfaces.First();
		}

		public static IPAddress GetInterfaceIPv4Address(NetworkInterface NetworkInterface)
		{
			var NetworkInterfaceProp = NetworkInterface.GetIPProperties();

			foreach (var Unicast in NetworkInterfaceProp.UnicastAddresses)
			{
				if (Unicast.Address.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork)
				{
					return Unicast.Address;
				}
			}

			throw new InvalidOperationException("Couldn't find interface IPv4 address.");
		}
	}

	public static class ObjectUtils
	{
		public static T Duplicate<T>(T SourceObject)
		{
			if (!typeof(T).IsSerializable)
			{
				throw new InvalidOperationException("Only serializable types are supported.");
			}

			var Formatter = new BinaryFormatter();
			using (var MemoryStream = new MemoryStream())
			{
				Formatter.Serialize(MemoryStream, SourceObject);
				MemoryStream.Seek(0, SeekOrigin.Begin);
				return (T)Formatter.Deserialize(MemoryStream);
			}
		}
	}
}
