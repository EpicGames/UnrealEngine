// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Management;
using System.Net;
using System.Net.NetworkInformation;
using System.Text;
using SwarmCoordinatorInterface;
using SwarmCommonUtils;

namespace SwarmCoordinator
{
    class Coordinator
    {
        static public bool AgentsDirty = true;

        static private Dictionary<string, AgentInfo> Agents = null;
        static private Int32 UniqueHandle = 0;
        static private DateTime LastMaintainTime = DateTime.MinValue;

        // Implementation of remote swarm coordinator interface
        public static bool Init()
        {
            Agents = new Dictionary<string, AgentInfo>();

            // Init the UniqueHandle to the number of seconds since 1st Jan 2000
            DateTime BaseTime = new DateTime( 2000, 1, 1 );
            TimeSpan Seconds = DateTime.UtcNow - BaseTime;
            UniqueHandle = ( Int32 )Seconds.TotalSeconds;

            return ( true );
        }

        public static bool Destroy()
        {
			lock( Agents )
			{
				Agents.Clear();
			}
            return ( true );
        }

		public static Int32 RestartAgentGroup( string GroupNameToRestart )
		{
			// The next time the coordinator receives a ping, it will request the client restart
			lock( Agents )
			{
				foreach( AgentInfo NextAgent in Agents.Values )
				{
					if( NextAgent.Configuration.ContainsKey( "GroupName" ) )
					{
						string GroupName = NextAgent.Configuration["GroupName"] as string;
						if( GroupName == GroupNameToRestart )
						{
							NextAgent.State = AgentState.Restarting;
						}
					}
				}
			}
			AgentsDirty = true;
			return 0;
		}
		
		public static void RestartAllAgents()
        {
            // The first time the coordinator receives a ping, it will request the client restart.
            // Clearing out the agents ensures all pings will be treated as the first.
			lock( Agents )
			{
				foreach( AgentInfo NextAgent in Agents.Values )
				{
					NextAgent.State = AgentState.Restarting;
				}
			}
            AgentsDirty = true;
        }

        /*
         * Registers an agent with the coordinator, or updates the ping time
         * 
         * Returns true if a restart is required
         */
		public static PingResponse Ping( AgentInfo UpdatedInfo )
        {
            AgentsDirty = true;

			// Add a known set of configuration items
            UpdatedInfo.Configuration["LastPingTime"] = DateTime.UtcNow;

			// Retreive information the agent can provide for us and supply defaults
			if( !UpdatedInfo.Configuration.ContainsKey( "GroupName" ) )
			{
				UpdatedInfo.Configuration["GroupName"] = "Default";
			}
			// If the Agent is working for someone, replace the local user name with
			// the user name on the machine that it's working for
			if( UpdatedInfo.Configuration.ContainsKey( "WorkingFor" ) )
			{
				// Look up the other user name
				string BossAgentName = UpdatedInfo.Configuration["WorkingFor"] as string;
				if( ( Agents.ContainsKey( BossAgentName ) ) &&
					( Agents[BossAgentName].Configuration.ContainsKey( "UserName" ) ) )
				{
					UpdatedInfo.Configuration["UserName"] = Agents[BossAgentName].Configuration["UserName"];
				}
			}
			else
			{
				UpdatedInfo.Configuration["WorkingFor"] = "";
			}
			if( !UpdatedInfo.Configuration.ContainsKey( "AssignedTo" ) )
			{
				UpdatedInfo.Configuration["AssignedTo"] = "";
                UpdatedInfo.Configuration["AssignedTime"] = DateTime.UtcNow;
			}
			if( !UpdatedInfo.Configuration.ContainsKey( "UserName" ) )
			{
				UpdatedInfo.Configuration["UserName"] = "UnknownUser";
			}

			// If the agent is remote, ping the machine to make sure we can talk to it
			// but only if the Agent didn't provide an IP address for us
			if( !UpdatedInfo.Configuration.ContainsKey( "IPAddress" ) )
			{
				// By default, we'll set the IP address to the loopback value
				UpdatedInfo.Configuration["IPAddress"] = IPAddress.Loopback;
				bool IPAddressUpdateSuccess = false;

				if( UpdatedInfo.Name != System.Net.Dns.GetHostName())
				{
					try
					{
						// Use the name, which can fail if DNS isn't enabled or if there's no host file
						Ping PingSender = new Ping();
						PingReply Reply = PingSender.Send( UpdatedInfo.Name );
						if( Reply.Status == IPStatus.Success )
						{
							// With a success, update the known IP address for this agent
							UpdatedInfo.Configuration["IPAddress"] = Reply.Address;
							IPAddressUpdateSuccess = true;
						}
					}
					catch( Exception )
					{
						// Any exception is a total failure
					}
				}

				// With a failure, update the agent's state to prevent usage
				if( !IPAddressUpdateSuccess )
				{
					UpdatedInfo.State = AgentState.Blocked;
				}
			}

			PingResponse Response;
			lock( Agents )
			{
				if( !Agents.ContainsKey( UpdatedInfo.Name ) )
				{
					// If this is a new agent, tell it to restart itself to pick up the latest version
					Response = PingResponse.Restart;
				}
				else if( Agents[UpdatedInfo.Name].State == AgentState.Restarting )
				{
					Response = PingResponse.Restart;

					// If this agent has been instructed to restart, keep overriding the
					// state until it's freed up to actually restart
					if( UpdatedInfo.State == AgentState.Available )
					{
						UpdatedInfo.State = AgentState.Restarted;
					}
					else
					{
						// Keep us in this cycle on the next ping
						UpdatedInfo.State = AgentState.Restarting;
					}
				}
				else
				{
					Response = PingResponse.Success;

					AgentInfo PreviousInfo = Agents[UpdatedInfo.Name];

					// If the WorkingFor changed, update the time
					string PreviousWorkingFor = PreviousInfo.Configuration["WorkingFor"] as string;
					string UpdatedWorkingFor = UpdatedInfo.Configuration["WorkingFor"] as string;
					
					DateTime UpdatedWorkingTime = DateTime.MaxValue;
					if( UpdatedWorkingFor != PreviousWorkingFor )
					{
                        UpdatedWorkingTime = DateTime.UtcNow;
					}
					else if( PreviousInfo.Configuration.ContainsKey( "WorkingTime" ) )
					{
						UpdatedWorkingTime = ( DateTime )PreviousInfo.Configuration["WorkingTime"];
					}
					UpdatedInfo.Configuration["WorkingTime"] = UpdatedWorkingTime;

					// Based on whether or not the agent is Working currently, update some state
					if( UpdatedInfo.State != AgentState.Working )
					{
						// No other agent should be AssignedTo it
						foreach( AgentInfo Agent in Agents.Values )
						{
							if( ( Agent.Configuration["AssignedTo"] as string ) == UpdatedInfo.Name )
							{
								Agent.Configuration["AssignedTo"] = "";
								Agent.Configuration["AssignedTime"] = DateTime.UtcNow;
							}
						}

						// If the previous state was Working for the same agent it's AssignedTo,
						// then we can clear the AssignedTo field
						if( PreviousInfo.State == AgentState.Working )
						{
							string PreviousAssignedTo = PreviousInfo.Configuration["AssignedTo"] as string;
							if( PreviousAssignedTo == PreviousWorkingFor )
							{
								PreviousInfo.Configuration["AssignedTo"] = "";
								PreviousInfo.Configuration["AssignedTime"] = DateTime.UtcNow;
							}
						}
					}
					else
					{
						// If it's WorkingFor someone, but not AssignedTo anyone else, update the AssignedTo
						if( ( ( UpdatedInfo.Configuration["WorkingFor"] as string ) != "" ) &&
							( ( PreviousInfo.Configuration["AssignedTo"] as string ) == "" ) )
						{
							PreviousInfo.Configuration["AssignedTo"] = UpdatedInfo.Configuration["WorkingFor"];
							PreviousInfo.Configuration["AssignedTime"] = DateTime.UtcNow;
						}
					}

					string UpdatedAssignedTo = PreviousInfo.Configuration["AssignedTo"] as string;
					UpdatedInfo.Configuration["AssignedTo"] = UpdatedAssignedTo;
					UpdatedInfo.Configuration["AssignedTime"] = DateTime.UtcNow;
				}

				// Set or reset the agent info to the latest
				Agents[UpdatedInfo.Name] = UpdatedInfo;
			}
			return Response;
        }

        /*
         * Returns a globally unique handle
         */
		private static Object GetUniqueHandleLock = new Object();
		public static Int32 GetUniqueHandle()
        {
			lock( GetUniqueHandleLock )
			{
				return( UniqueHandle++ );
			}
        }

		// Available < Busy < Other (Working)
		private static int AliveAgentSorter( AgentInfo A, AgentInfo B )
		{
			if( A.State == AgentState.Available )
			{
				if( B.State == AgentState.Available )
				{
					return 0;
				}
				else
				{
					return -1;
				}
			}
			else
			{
				if( B.State == AgentState.Available )
				{
					return 1;
				}
				else
				{
					// Neither are Available, sort on Busy
					if( A.State == AgentState.Busy )
					{
						if( B.State == AgentState.Busy )
						{
							return 0;
						}
						else
						{
							return -1;
						}
					}
					else
					{
						if( B.State == AgentState.Busy )
						{
							return 1;
						}
						else
						{
							return 0;
						}
					}

				}
			}
		}

        /*
         * Returns a list of agents that have an active ping and the same version info
         * 
         * A requested Version of 0.0.0.0 will return all agents
         */
		public static List<AgentInfo> GetAvailableAgents( Hashtable RequestedConfiguration )
        {
			// Validate the requested configuration
			if( RequestedConfiguration.ContainsKey( "Version" ) == false )
			{
				return null;
			}

			AgentInfo RequestingAgent;
			IPAddress MyAddressFromRequestingAgentPOV;

			// If the requestor is Blocked, ignore the request
			if (!RequestedConfiguration.ContainsKey("RequestingAgentName"))
			{
				return null;
			}

			// If the requesting agent is not an approved state, block this request
			string RequestingAgentName = RequestedConfiguration["RequestingAgentName"] as string;
			if ((Agents.TryGetValue(RequestingAgentName, out RequestingAgent) &&
				((RequestingAgent.State != AgentState.Available) &&
					(RequestingAgent.State != AgentState.Working) &&
					(RequestingAgent.State != AgentState.Busy))))
			{
				return null;
			}

			var BestInt = NetworkUtils.GetBestInterface(RequestingAgent.Configuration["IPAddress"] as IPAddress);
			MyAddressFromRequestingAgentPOV = NetworkUtils.GetInterfaceIPv4Address(BestInt);

			// Extract the requested configuration
			Version RequestedVersion = RequestedConfiguration["Version"] as Version;

			List<AgentInfo> AliveAgents = new List<AgentInfo>();
			lock( Agents )
			{
				foreach( AgentInfo Agent in Agents.Values )
				{
					// Only match against major and minor versions
					bool VersionMatch =
						( RequestedVersion.Major < Agent.Version.Major ) ||
						( ( RequestedVersion.Major == Agent.Version.Major ) &&
						  ( RequestedVersion.Minor <= Agent.Version.Minor ) );

					bool StateMatch =
						( Agent.State == AgentState.Available ) ||
						( Agent.State == AgentState.Working ) ||
						( Agent.State == AgentState.Busy );

					// If there is no group name, everything matches, otherwise
					// the group names must match
					bool GroupMatch = true;
					if( ( RequestedConfiguration.ContainsKey( "GroupName" ) ) &&
						( Agent.Configuration.ContainsKey( "GroupName" ) ) )
					{
						string RequestedGroupName = RequestedConfiguration["GroupName"] as string;
						string AgentGroupName = Agent.Configuration["GroupName"] as string;
						GroupMatch = ( RequestedGroupName == AgentGroupName );
					}

					// If all conditions are met, we match
					if( VersionMatch && StateMatch && GroupMatch )
					{
						if (MyAddressFromRequestingAgentPOV != IPAddress.Loopback &&
							(Agent.Configuration["IPAddress"] as IPAddress).Equals(IPAddress.Loopback))
						{
							var NewAgentInfo = ObjectUtils.Duplicate(Agent);

							NewAgentInfo.Configuration["IPAddress"] = MyAddressFromRequestingAgentPOV;

							AliveAgents.Add(NewAgentInfo);
						}
						else
						{
							AliveAgents.Add(Agent);
						}
					}
				}

				// If the requesting agent is asking for assignment, do it now that we have the right set
				bool ValidRequestForAssignment = false;
				if( RequestedConfiguration.ContainsKey( "RequestAssignmentFor" ) )
				{
					// If the requesting agent is not an approved state, block this request
					ValidRequestForAssignment = (RequestingAgent.State == AgentState.Available) ||
												(RequestingAgent.State == AgentState.Working) ||
												(RequestingAgent.State == AgentState.Busy);
				}

				if( ValidRequestForAssignment )
				{
					// Assign all Available agents to this requesting agent and bin the Working ones
					Int32 NewlyAssignedCount = 0;
					Int32 AlreadyAssignedCount = 0;
					Dictionary<string, List<AgentInfo>> AssignedToBins = new Dictionary<string, List<AgentInfo>>();

					foreach( AgentInfo Agent in AliveAgents )
					{
						if( Agent.State == AgentState.Available )
						{
							Agent.Configuration["AssignedTo"] = RequestingAgent.Name;
							Agent.Configuration["AssignedTime"] = DateTime.UtcNow;
							NewlyAssignedCount++;
						}
						else if( Agent.State == AgentState.Working )
						{
							// Collect the already assigned agents
							string AssignedTo = Agent.Configuration["AssignedTo"] as string;
							if( AssignedTo != Agent.Name )
							{
								List<AgentInfo> AgentBin;
								if( !AssignedToBins.TryGetValue( AssignedTo, out AgentBin ) )
								{
									AgentBin = new List<AgentInfo>();
									AssignedToBins[AssignedTo] = AgentBin;
								}
								AgentBin.Add( Agent );
								AlreadyAssignedCount++;
							}
						}
					}

					// If the ratio of assigned agents is already 1:1, there's nothing more
					// we can do, regardless of how many we got previously
					if( AlreadyAssignedCount != AssignedToBins.Count )
					{
						// As long as reassigning an agent would not push us above the average, do it
						while( ( NewlyAssignedCount + 1 ) <= ( ( float )( AlreadyAssignedCount - 1 ) / ( float )AssignedToBins.Count ) )
						{
							// Search for the largest bin to reassign an agent from
							List<AgentInfo> LargestBin = null;
							Int32 LargestBinCount = 0;
							foreach( List<AgentInfo> Bin in AssignedToBins.Values )
							{
								if( LargestBinCount < Bin.Count )
								{
									LargestBinCount = Bin.Count;
									LargestBin = Bin;
								}
							}
							// Search for the newest WorkingTime
							DateTime MaxWorkingTime = ( DateTime )LargestBin[0].Configuration["WorkingTime"];
							Int32 ReassignedAgentIndex = 0;
							for( Int32 i = 1; i < LargestBin.Count; i++ )
							{
								DateTime NextWorkingTime = ( DateTime )LargestBin[i].Configuration["WorkingTime"];
								if( MaxWorkingTime < NextWorkingTime )
								{
									MaxWorkingTime = NextWorkingTime;
									ReassignedAgentIndex = i;
								}
							}
							AgentInfo ReassignedAgent = LargestBin[ReassignedAgentIndex];
							LargestBin.RemoveAt( ReassignedAgentIndex );

							// Reassigned and adjust the counts
							ReassignedAgent.Configuration["AssignedTo"] = RequestingAgent.Name;
							ReassignedAgent.Configuration["AssignedTime"] = DateTime.UtcNow;
							AlreadyAssignedCount--;
							NewlyAssignedCount++;
						}
					}

					// Resort the list -> (Available, Busy, Other (Working))
					AliveAgents.Sort( AliveAgentSorter );
				}
			}
            return ( AliveAgents );
        }

        // Maintain the active agent list
        public static void MaintainAgents()
        {
            // Only maintain the agents periodically
            if( DateTime.UtcNow > LastMaintainTime + TimeSpan.FromSeconds(20) )
            {
                LastMaintainTime = DateTime.UtcNow;
                lock( Agents )
				{
					foreach( AgentInfo Agent in Agents.Values )
					{
						DateTime LastPingTime = ( DateTime )Agent.Configuration["LastPingTime"];

						// Only attempt to update the agent state if it's not already set
						// to a permanent resting state such as Dead or Closed
						if( ( Agent.State != AgentState.Dead ) &&
							( Agent.State != AgentState.Closed ) )
						{
							// Update the agent state based on the ping time
							// If it's been more than 5 minutes, go into Starving mode
							if( DateTime.UtcNow > LastPingTime + TimeSpan.FromMinutes(5) )
							{
								Agent.State = AgentState.Starving;
								AgentsDirty = true;

								// If longer than 10 minutes, consider it Dead
								if( DateTime.UtcNow > LastPingTime + TimeSpan.FromMinutes(10) )
								{
									Agent.State = AgentState.Dead;
								}
							}
						}
					}
				}
			}
        }

		public static Int32 Method( Hashtable InParameters, ref Hashtable OutParameters )
		{
			// Currently, this method doesn't do anything
			return 0;
		}

        // Local functions to display info about agents in the UI
        public static Dictionary<string, AgentInfo> GetAgents()
        {
			Dictionary<string, AgentInfo> CopyOfAgents;
			lock( Agents )
			{
				CopyOfAgents = new Dictionary<string,AgentInfo>( Agents );
			}
			return CopyOfAgents;
        }
    }
}
