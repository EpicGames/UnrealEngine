// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
#if !__MonoCS__
using System.Deployment.Application;
#endif
using System.Windows.Forms;

namespace SwarmCoordinator
{
    static class Program
    {
		/**
		 * Checks for any pending updates for this application, and if it finds
		 * any, updates the application as part of a restart
		 */
		static void CheckForUpdate()
		{
			try
			{
#if !__MonoCS__
				if( ApplicationDeployment.IsNetworkDeployed )
				{
					ApplicationDeployment Current = ApplicationDeployment.CurrentDeployment;
					if( Current.CheckForUpdate() )
					{
						Current.Update();
					}
				}
#endif
			}
			catch( Exception )
			{
			}
		}

        /// <summary>
        /// The main entry point for the application.
        /// </summary>
        [STAThread]
        static void Main()
        {
            Application.EnableVisualStyles();
            Application.SetCompatibleTextRenderingDefault( false );

            // Create the window
            SwarmCoordinator MainWindow = new SwarmCoordinator();
            MainWindow.Init();

            while( MainWindow.Ticking )
            {
                Application.DoEvents();
                MainWindow.Run();

                // Yield a little time to the system
                System.Threading.Thread.Sleep( 100 );
            }

            MainWindow.Destroy();

			if( MainWindow.Restarting() )
			{
				CheckForUpdate();
				Application.Restart();
			}
        }
    }
}