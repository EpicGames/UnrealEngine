<?xml version='1.0' encoding='utf-8'?>
<SettingsFile xmlns="http://schemas.microsoft.com/VisualStudio/2004/01/settings" CurrentProfile="(Default)" GeneratedClassNamespace="SwarmCoordinator.Properties" GeneratedClassName="Settings">
  <Profiles />
  <Settings>
    <Setting Name="CoordinatorRemotingPort" Type="System.Int32" Scope="Application">
      <Value Profile="(Default)">8009</Value>
    </Setting>
  </Settings>
</SettingsFile>