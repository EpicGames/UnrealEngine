// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Diagnostics;
using System.Drawing;
using System.Runtime.Remoting;
using System.Runtime.Remoting.Channels;
using System.Runtime.Remoting.Channels.Tcp;
using System.Text;
using System.Windows.Forms;
using SwarmCoordinatorInterface;

namespace SwarmCoordinator
{
    public partial class SwarmCoordinator : Form
    {
        public bool Ticking = false;
		public bool RestartRequested = false;

        private TcpServerChannel ServerChannel = null;

        public SwarmCoordinator()
        {
            InitializeComponent();
        }

        private void Log( string Message )
        {
            Debug.WriteLine( DateTime.Now.ToLongTimeString() + ": " + Message );
        }

        private void AppExceptionHandler( object sender, UnhandledExceptionEventArgs args )
        {
            Exception E = ( Exception )args.ExceptionObject;
            Log( "Application exception: " + E.ToString() );
        }

        public void Init()
        {
            // Register application exception handler
            AppDomain.CurrentDomain.UnhandledException += new UnhandledExceptionEventHandler( AppExceptionHandler );

            // Register in interface implementation
            RemotingConfiguration.RegisterWellKnownServiceType( typeof( SwarmCoordinatorImplementation ), "SwarmCoordinator", WellKnownObjectMode.Singleton );

            // Start up network services, by opening a network communication channel
            ServerChannel = new TcpServerChannel( "CoordinatorServer", Properties.Settings.Default.CoordinatorRemotingPort );
            ChannelServices.RegisterChannel( ServerChannel, false );
               
            // Initialise the interface
            Coordinator.Init();

			// Set the title bar to include the name of the machine
			TopLevelControl.Text = "Swarm Coordinator running on " + System.Net.Dns.GetHostName();

            // Display the window
            Show();

            Ticking = true;
        }

		public bool Restarting()
		{
			return RestartRequested;
		}

        public void Destroy()
        {
            // Clean up the coordinator
            Coordinator.Destroy();

            // Free up the remoting channel
            if( ServerChannel != null )
            {
                ChannelServices.UnregisterChannel( ServerChannel );
                ServerChannel = null;
            }
        }

        public void Run()
        {
            // Update the agent state based on the ping time
            Coordinator.MaintainAgents();

            // Update the grid view to represent to current state of the agents
            UpdateGridRows();
        }

        private void UpdateGridRows()
        {
            if( Coordinator.AgentsDirty )
            {
                // Clear out any existing rows
                AgentGridView.Rows.Clear();

                // Get a list of agents
                Dictionary<string, AgentInfo> Agents = Coordinator.GetAgents();
				List<string> AgentNames = new List<string>( Agents.Keys );
				AgentNames.Sort();

                // Create a row for each agent
				foreach( string AgentName in AgentNames )
				{
					AgentInfo NextAgent = Agents[AgentName];
                    DataGridViewRow Row = new DataGridViewRow();
					DataGridViewTextBoxCell NextCell = null;

                    NextCell = new DataGridViewTextBoxCell();
					NextCell.Value = NextAgent.Name;
					if( NextAgent.Configuration.ContainsKey( "UserName" ) )
					{
						NextCell.Value += " (" + NextAgent.Configuration["UserName"] + ")";
					}
					else
					{
						NextCell.Value += " (UnknownUser)";
					}
					Row.Cells.Add( NextCell );

					NextCell = new DataGridViewTextBoxCell();
					if( NextAgent.Configuration.ContainsKey( "GroupName" ) )
					{
						NextCell.Value = NextAgent.Configuration["GroupName"];
					}
					else
					{
						NextCell.Value = "Undefined";
					}
					Row.Cells.Add( NextCell );

					NextCell = new DataGridViewTextBoxCell();
					NextCell.Value = NextAgent.Version;
					Row.Cells.Add( NextCell );

					NextCell = new DataGridViewTextBoxCell();
					string NextCellStateValue;
					if( NextAgent.State == AgentState.Working )
					{
						if( ( NextAgent.Configuration.ContainsKey( "WorkingFor" ) ) &&
							( ( NextAgent.Configuration["WorkingFor"] as string ) != "" ) )
						{
							NextCellStateValue = "Working for " + NextAgent.Configuration["WorkingFor"];
						}
						else
						{
							NextCellStateValue = "Working for an unknown Agent";
						}
					}
					else
					{
						NextCellStateValue = NextAgent.State.ToString();
					}
					if( ( NextAgent.Configuration.ContainsKey( "AssignedTo" ) ) &&
						( ( NextAgent.Configuration["AssignedTo"] as string ) != "" ) )
					{
						NextCellStateValue += ", Assigned to " + NextAgent.Configuration["AssignedTo"];
					}
					else
					{
						NextCellStateValue += ", Unassigned";
					}
					NextCell.Value = NextCellStateValue;
					Row.Cells.Add( NextCell );

					NextCell = new DataGridViewTextBoxCell();
					if( NextAgent.Configuration.ContainsKey( "IPAddress" ) )
					{
						NextCell.Value = NextAgent.Configuration["IPAddress"];
					}
					else
					{
						NextCell.Value = "Undefined";
					}
					Row.Cells.Add( NextCell );

					NextCell = new DataGridViewTextBoxCell();
					if( NextAgent.Configuration.ContainsKey( "LastPingTime" ) )
					{
						NextCell.Value = NextAgent.Configuration["LastPingTime"];
					}
					else
					{
						NextCell.Value = "Undefined";
					}
					Row.Cells.Add( NextCell );

					NextCell = new DataGridViewTextBoxCell();
					if( NextAgent.Configuration.ContainsKey( "LocalAvailableCores" ) )
					{
						NextCell.Value = NextAgent.Configuration["LocalAvailableCores"];
					}
					else
					{
						NextCell.Value = "Undefined";
					}
					Row.Cells.Add( NextCell );

					NextCell = new DataGridViewTextBoxCell();
					if( NextAgent.Configuration.ContainsKey( "RemoteAvailableCores" ) )
					{
						NextCell.Value = NextAgent.Configuration["RemoteAvailableCores"];
					}
					else
					{
						NextCell.Value = "Undefined";
					}
					Row.Cells.Add( NextCell );

					AgentGridView.Rows.Add( Row );
                }

                Coordinator.AgentsDirty = false;
            }
        }

        private void SwarmCoordinator_Closing( object sender, FormClosingEventArgs e )
        {
            Ticking = false;
        }

		private void RestartCoordinatorButton_Click( object sender, EventArgs e )
		{
			RestartRequested = true;
			Ticking = false;
		}

		private void RestartQAAgentsButton_Click( object sender, EventArgs e )
		{
			Coordinator.RestartAgentGroup( "QATestGroup" );
		}

		private void RestartAllAgentsButton_Click( object sender, EventArgs e )
		{
			Coordinator.RestartAllAgents();
		}
    }

    public class SwarmCoordinatorImplementation : MarshalByRefObject, ISwarmCoordinator
    {
        public PingResponse Ping( AgentInfo Agent )
        {
            return ( Coordinator.Ping( Agent ) );
        }

        public Int32 GetUniqueHandle()
        {
            return ( Coordinator.GetUniqueHandle() );
        }

		public List<AgentInfo> GetAvailableAgents( Hashtable RequestedConfiguration )
        {
			return ( Coordinator.GetAvailableAgents( RequestedConfiguration ) );
        }

		public Int32 RestartAgentGroup( string GroupNameToRestart )
		{
			return ( Coordinator.RestartAgentGroup( GroupNameToRestart ) );
		}

		public Int32 Method( Hashtable InParameters, ref Hashtable OutParameters )
		{
			return ( Coordinator.Method( InParameters, ref OutParameters ) );
		}
	}
}