<?xml version="1.0" encoding="utf-8"?>
<Project DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003" ToolsVersion="12.0">
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">AnyCPU</Platform>
    <ProductVersion>9.0.30729</ProductVersion>
    <SchemaVersion>2.0</SchemaVersion>
    <ProjectGuid>{FA3182A3-4755-4E12-863E-EF51D637342E}</ProjectGuid>
    <OutputType>WinExe</OutputType>
    <AppDesignerFolder>Properties</AppDesignerFolder>
    <RootNamespace>SwarmCoordinator</RootNamespace>
    <AssemblyName>SwarmCoordinator</AssemblyName>
    <SccProjectName>
    </SccProjectName>
    <SccLocalPath>
    </SccLocalPath>
    <SccAuxPath>
    </SccAuxPath>
    <SccProvider>
    </SccProvider>
    <FileUpgradeFlags>
    </FileUpgradeFlags>
    <OldToolsVersion>3.5</OldToolsVersion>
    <UpgradeBackupLocation>
    </UpgradeBackupLocation>
    <TargetFrameworkVersion>v4.6.2</TargetFrameworkVersion>
    <IsWebBootstrapper>false</IsWebBootstrapper>
    <SignManifests>false</SignManifests>
    <SignAssembly>false</SignAssembly>
    <ManifestCertificateThumbprint>E10B5956C0FAFF8479C61A48C48EFDE555A18633</ManifestCertificateThumbprint>
    <ManifestKeyFile>
    </ManifestKeyFile>
    <GenerateManifests>true</GenerateManifests>
    <TargetFrameworkProfile>
    </TargetFrameworkProfile>
    <PublishUrl>\\devweb-02\Deploy\Swarm\SwarmCoordinator\</PublishUrl>
    <Install>true</Install>
    <InstallFrom>Unc</InstallFrom>
    <UpdateEnabled>false</UpdateEnabled>
    <UpdateMode>Foreground</UpdateMode>
    <UpdateInterval>7</UpdateInterval>
    <UpdateIntervalUnits>Days</UpdateIntervalUnits>
    <UpdatePeriodically>false</UpdatePeriodically>
    <UpdateRequired>false</UpdateRequired>
    <MapFileExtensions>true</MapFileExtensions>
    <ProductName>Swarm Coordinator</ProductName>
    <PublisherName>Startup</PublisherName>
    <CreateWebPageOnPublish>true</CreateWebPageOnPublish>
    <WebPage>publish.htm</WebPage>
    <OpenBrowserOnPublish>false</OpenBrowserOnPublish>
    <ApplicationRevision>57</ApplicationRevision>
    <ApplicationVersion>1.0.0.%2a</ApplicationVersion>
    <UseApplicationTrust>false</UseApplicationTrust>
    <PublishWizardCompleted>true</PublishWizardCompleted>
    <BootstrapperEnabled>true</BootstrapperEnabled>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <DebugSymbols>true</DebugSymbols>
    <DebugType>pdbonly</DebugType>
    <Optimize>false</Optimize>
    <OutputPath>..\..\..\..\Binaries\DotNET\</OutputPath>
    <DefineConstants>DEBUG;TRACE</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
    <AllowUnsafeBlocks>false</AllowUnsafeBlocks>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Development|AnyCPU' ">
    <DebugType>pdbonly</DebugType>
    <Optimize>true</Optimize>
    <OutputPath>..\..\..\..\Binaries\DotNET\</OutputPath>
    <DefineConstants>TRACE</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
    <AllowUnsafeBlocks>false</AllowUnsafeBlocks>
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="System" />
    <Reference Include="System.Core">
      <RequiredTargetFramework>3.5</RequiredTargetFramework>
    </Reference>
    <Reference Include="System.Data" />
    <Reference Include="System.Deployment" />
    <Reference Include="System.Drawing" />
    <Reference Include="System.Management" />
    <Reference Include="System.Runtime.Remoting" />
    <Reference Include="System.Windows.Forms" />
    <Reference Include="System.XML" />
  </ItemGroup>
  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
    <Compile Include="Coordinator.cs" />
    <Compile Include="SwarmCoordinator.cs">
      <SubType>Form</SubType>
    </Compile>
    <Compile Include="SwarmCoordinator.Designer.cs">
      <DependentUpon>SwarmCoordinator.cs</DependentUpon>
    </Compile>
    <Compile Include="Program.cs" />
    <Compile Include="Properties\AssemblyInfo.cs" />
    <EmbeddedResource Include="Properties\Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
      <SubType>Designer</SubType>
    </EmbeddedResource>
    <EmbeddedResource Include="SwarmCoordinator.resx">
      <SubType>Designer</SubType>
      <DependentUpon>SwarmCoordinator.cs</DependentUpon>
    </EmbeddedResource>
    <Compile Include="Properties\Resources.Designer.cs">
      <AutoGen>True</AutoGen>
      <DependentUpon>Resources.resx</DependentUpon>
      <DesignTime>True</DesignTime>
    </Compile>
    <None Include="app.config" />
    <None Include="Properties\Settings.settings">
      <Generator>SettingsSingleFileGenerator</Generator>
      <LastGenOutput>Settings.Designer.cs</LastGenOutput>
    </None>
    <Compile Include="Properties\Settings.Designer.cs">
      <AutoGen>True</AutoGen>
      <DependentUpon>Settings.settings</DependentUpon>
      <DesignTimeSharedInput>True</DesignTimeSharedInput>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\SwarmCommonUtils\SwarmCommonUtils.csproj">
      <Project>{65521250-b022-442f-b961-3af5ada02083}</Project>
      <Name>SwarmCommonUtils</Name>
    </ProjectReference>
    <ProjectReference Include="..\SwarmCoordinatorInterface\SwarmCoordinatorInterface.csproj">
      <Project>{4C299DD0-B968-49F2-894A-9F115C369E72}</Project>
      <Name>SwarmCoordinatorInterface</Name>
    </ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <BootstrapperPackage Include=".NETFramework,Version=v4.0,Profile=Client">
      <Visible>False</Visible>
      <ProductName>Microsoft .NET Framework 4 Client Profile %28x86 and x64%29</ProductName>
      <Install>true</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Client.3.5">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1 Client Profile</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.2.0">
      <Visible>False</Visible>
      <ProductName>.NET Framework 2.0 %28x86%29</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.0">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.0 %28x86%29</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.5">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.5.SP1">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Windows.Installer.3.1">
      <Visible>False</Visible>
      <ProductName>Windows Installer 3.1</ProductName>
      <Install>true</Install>
    </BootstrapperPackage>
  </ItemGroup>
  <Import Project="$(MSBuildBinPath)\Microsoft.CSharp.targets" />
  <!-- To modify your build process, add your task inside one of the targets below and uncomment it. 
       Other similar extension points exist, see Microsoft.Common.targets.
  <Target Name="BeforeBuild">
  </Target>
  <Target Name="AfterBuild">
  </Target>
  -->
</Project>