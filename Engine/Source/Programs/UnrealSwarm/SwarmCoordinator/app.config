<?xml version="1.0"?>
<configuration>
    <configSections>
        <sectionGroup name="applicationSettings" type="System.Configuration.ApplicationSettingsGroup, System, Version=2.0.0.0, Culture=neutral, PublicKeyToken=b77a5c561934e089">
            <section name="SwarmCoordinator.Properties.Settings" type="System.Configuration.ClientSettingsSection, System, Version=2.0.0.0, Culture=neutral, PublicKeyToken=b77a5c561934e089" requirePermission="false"/>
        </sectionGroup>
    </configSections>
    <applicationSettings>
        <SwarmCoordinator.Properties.Settings>
            <setting name="CoordinatorRemotingPort" serializeAs="String">
                <value>8009</value>
            </setting>
        </SwarmCoordinator.Properties.Settings>
    </applicationSettings>
<startup><supportedRuntime version="v4.0" sku=".NETFramework,Version=v4.6.2"/></startup></configuration>
