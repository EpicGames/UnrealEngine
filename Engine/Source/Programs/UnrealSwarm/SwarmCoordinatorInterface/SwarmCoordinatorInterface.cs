// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Reflection;
using System.Runtime.Remoting;
using System.Runtime.Remoting.Channels;
using System.Text;

namespace SwarmCoordinatorInterface
{
    // The info sent by a ping per machine and returned to the instigators
    // (this will probably be split up)
    [Serializable]
    public enum AgentState
    {
        Unknown		= 0,
        Available	= 1,
        Working		= 2,
        Blocked		= 3,
        Starving	= 4,
        Dead		= 5,
		Standalone	= 6,
		Closed		= 7,
		Restarting	= 8,
		Restarted	= 9,
		Busy		= 10,
	}

    [Serializable]
    public class AgentInfo
    {
		// Constant agent attributes
		public string Name = System.Net.Dns.GetHostName();
		public AgentState State = AgentState.Unknown;
		public Version Version = new Version( 0, 0, 0, 0 );
		public Hashtable Configuration = new Hashtable();
	}

	[Serializable]
	public enum PingResponse
	{
		Success,
		Restart
	}

    public interface ISwarmCoordinator
    {
        /*
         * Registers an agent with the coordinator, or updates the ping time
         * 
         * Returns true if a restart is required
         */
		PingResponse Ping( AgentInfo UpdatedInfo );

        /*
         * Returns a globally unique handle
         */
        Int32 GetUniqueHandle();

        /*
         * Returns a list of agents that have an active ping and the same version info
         * 
         * A RequestedVersion of 0.0.0.0 will return all agents
         */
        List<AgentInfo> GetAvailableAgents( Hashtable RequestedConfiguration );

		/*
		 * Requests a restart for all agents belonging to a specific group
		 */
		Int32 RestartAgentGroup( string GroupNameToRestart );

		/*
		 * A fully general interface to the Coordinator which is used for extending the API
		 * in novel ways, debugging interfaces, extended statistics gathering, etc.
		 */
		Int32 Method( Hashtable InParameters, ref Hashtable OutParameters );
    }
}
